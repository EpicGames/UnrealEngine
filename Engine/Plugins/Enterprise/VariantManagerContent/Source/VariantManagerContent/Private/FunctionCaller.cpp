// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionCaller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionCaller)

#if WITH_EDITORONLY_DATA

#include "LevelVariantSets.h"
#include "Variant.h"
#include "VariantSet.h"

#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_FunctionEntry.h"

// This file is based on MovieSceneEvent.cpp

UK2Node_FunctionEntry* FFunctionCaller::GetFunctionEntry() const
{
	return CastChecked<UK2Node_FunctionEntry>(FunctionEntry.Get(), ECastCheckedType::NullAllowed);
}

void FFunctionCaller::SetFunctionEntry(UK2Node_FunctionEntry* InFunctionEntry)
{
	FunctionEntry = InFunctionEntry;
	CacheFunctionName();
}

bool FFunctionCaller::CanHaveArguments() const
{
	if (IsBoundToBlueprint())
	{
		if (UK2Node_FunctionEntry* Function = GetFunctionEntry())
		{
			return CanHaveArguments(Function);
		}
	}

	return false;
}

namespace
{
	bool IsTargetPinValid(const TSharedPtr<FUserPinInfo>& TargetPin)
	{
		return TargetPin.IsValid() &&
			!TargetPin->PinType.bIsReference &&
			(TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
			TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface);
	}

	bool IsLevelVariantSetsPinValid(const TSharedPtr<FUserPinInfo>& TargetPin)
	{
		return TargetPin.IsValid() &&
			!TargetPin->PinType.bIsReference &&
			TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object &&
			TargetPin->PinType.PinSubCategoryObject == ULevelVariantSets::StaticClass();
	}

	bool IsVariantSetPinValid(const TSharedPtr<FUserPinInfo>& TargetPin)
	{
		return TargetPin.IsValid() &&
			!TargetPin->PinType.bIsReference &&
			TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object &&
			TargetPin->PinType.PinSubCategoryObject == UVariantSet::StaticClass();
	}

	bool IsVariantPinValid(const TSharedPtr<FUserPinInfo>& TargetPin)
	{
		return TargetPin.IsValid() &&
			!TargetPin->PinType.bIsReference &&
			TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object &&
			TargetPin->PinType.PinSubCategoryObject == UVariant::StaticClass();
	}

	bool IsArgumentsPinValid(const TSharedPtr<FUserPinInfo>& TargetPin)
	{
		return TargetPin.IsValid() &&
			!TargetPin->PinType.bIsReference &&
			TargetPin->PinType.IsMap() &&
			TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name &&
			TargetPin->PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_String;
	}

}

bool FFunctionCaller::CanHaveArguments(UK2Node_FunctionEntry* Function)
{
	if (Function)
	{
		if (Function->UserDefinedPins.Num() == 2)
		{
			TSharedPtr<FUserPinInfo> TargetPin		= Function->UserDefinedPins[0];
			TSharedPtr<FUserPinInfo> ArgumentsPin	= Function->UserDefinedPins[1];

			return IsTargetPinValid(TargetPin) && IsArgumentsPinValid(ArgumentsPin);
		}
		else if (Function->UserDefinedPins.Num() == 5)
		{
			TSharedPtr<FUserPinInfo> TargetPin			 = Function->UserDefinedPins[0];
			TSharedPtr<FUserPinInfo> LevelVariantSetsPin = Function->UserDefinedPins[1];
			TSharedPtr<FUserPinInfo> VariantSetPin		 = Function->UserDefinedPins[2];
			TSharedPtr<FUserPinInfo> VariantPin			 = Function->UserDefinedPins[3];
			TSharedPtr<FUserPinInfo> ArgumentsPin		 = Function->UserDefinedPins[4];

			return IsTargetPinValid(TargetPin) &&
				IsLevelVariantSetsPinValid(LevelVariantSetsPin) &&
				IsVariantSetPinValid(VariantSetPin) &&
				IsVariantPinValid(VariantPin) &&
				IsArgumentsPinValid(ArgumentsPin);
		}
	}

	return false;
}

bool FFunctionCaller::IsBoundToBlueprint() const
{
	return IsValidFunction(GetFunctionEntry());
}

bool FFunctionCaller::IsValidFunction(UK2Node_FunctionEntry* Function)
{
	if (!IsValid(Function) || !IsValid(Function->GetGraph()))
	{
		return false;
	}
	else if (Function->UserDefinedPins.Num() == 0)
	{
		return true;
	}
	else if (Function->UserDefinedPins.Num() == 1)
	{
		return IsTargetPinValid(Function->UserDefinedPins[0]);
	}
	else if (Function->UserDefinedPins.Num() == 2)
	{
		TSharedPtr<FUserPinInfo> TargetPin		= Function->UserDefinedPins[0];
		TSharedPtr<FUserPinInfo> ArgumentsPin	= Function->UserDefinedPins[1];

		return IsTargetPinValid(TargetPin) && IsArgumentsPinValid(ArgumentsPin);
	}
	else if (Function->UserDefinedPins.Num() == 4)
	{
		TSharedPtr<FUserPinInfo> TargetPin			 = Function->UserDefinedPins[0];
		TSharedPtr<FUserPinInfo> LevelVariantSetsPin = Function->UserDefinedPins[1];
		TSharedPtr<FUserPinInfo> VariantSetPin		 = Function->UserDefinedPins[2];
		TSharedPtr<FUserPinInfo> VariantPin			 = Function->UserDefinedPins[3];

		return IsTargetPinValid(TargetPin) &&
			IsLevelVariantSetsPinValid(LevelVariantSetsPin) &&
			IsVariantSetPinValid(VariantSetPin) &&
			IsVariantPinValid(VariantPin);
	}
	else if (Function->UserDefinedPins.Num() == 5)
	{
		TSharedPtr<FUserPinInfo> TargetPin			 = Function->UserDefinedPins[0];
		TSharedPtr<FUserPinInfo> LevelVariantSetsPin = Function->UserDefinedPins[1];
		TSharedPtr<FUserPinInfo> VariantSetPin		 = Function->UserDefinedPins[2];
		TSharedPtr<FUserPinInfo> VariantPin			 = Function->UserDefinedPins[3];
		TSharedPtr<FUserPinInfo> ArgumentsPin		 = Function->UserDefinedPins[4];

		return IsTargetPinValid(TargetPin) &&
			IsLevelVariantSetsPinValid(LevelVariantSetsPin) &&
			IsVariantSetPinValid(VariantSetPin) &&
			IsVariantPinValid(VariantPin) &&
			IsArgumentsPinValid(ArgumentsPin);
	}

	return false;
}

uint32 FFunctionCaller::GetDisplayOrder() const
{
	return DisplayOrder;
}

void FFunctionCaller::SetDisplayOrder(uint32 InDisplayOrder)
{
	DisplayOrder = InDisplayOrder;
}

void FFunctionCaller::CacheFunctionName()
{
	UK2Node_FunctionEntry* Node = GetFunctionEntry();

	FunctionName = NAME_None;

	if (IsValidFunction(Node))
	{
		UEdGraph* Graph = Node->GetGraph();
		if (Graph)
		{
			FunctionName = Graph->GetFName();
		}
	}
}

#endif // WITH_EDITORONLY_DATA


void FFunctionCaller::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		CacheFunctionName();
	}
#endif
}


