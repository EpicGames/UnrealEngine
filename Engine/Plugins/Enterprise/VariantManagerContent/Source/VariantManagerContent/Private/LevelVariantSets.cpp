// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSets.h"

#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "VariantSet.h"
#include "Variant.h"
#include "VariantObjectBinding.h"
#include "LevelVariantSetsFunctionDirector.h"
#include "Kismet/GameplayStatics.h"
#include "LevelVariantSetsActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelVariantSets)

#if WITH_EDITOR
#include "Editor.h"
#include "GameDelegates.h"
#include "Engine/Blueprint.h"
#else
#include "Engine/World.h"
#include "UObject/Package.h"
#endif

#define LOCTEXT_NAMESPACE "LevelVariantSets"

namespace UE
{
	namespace LevelVariantSets
	{
		namespace Private
		{
			/** Makes it so that all others variants that depend on a variant of 'VariantSet' have those particular dependencies deleted */
			void ResetVariantSetDependents( UVariantSet* VariantSet )
			{
				if ( !VariantSet )
				{
					return;
				}

				ULevelVariantSets* LevelVariantSets = VariantSet->GetTypedOuter<ULevelVariantSets>();
				if ( !LevelVariantSets )
				{
					return;
				}

				for ( UVariant* Variant : VariantSet->GetVariants() )
				{
					const bool bOnlyEnabledDependencies = false;
					for ( UVariant* Dependent : Variant->GetDependents( LevelVariantSets, bOnlyEnabledDependencies ) )
					{
						for ( int32 DependencyIndex = Dependent->GetNumDependencies() - 1; DependencyIndex >= 0; --DependencyIndex )
						{
							FVariantDependency& Dependency = Dependent->GetDependency( DependencyIndex );
							UVariant* TargetVariant = Dependency.Variant.Get();
							if ( TargetVariant == Variant )
							{
								// Delete the entire dependency because we can't leave a dependency without a valid Variant selected or
								// we may run into minor slightly awkward states (i.e. not being able to pick *any* variant)
								Dependent->DeleteDependency( DependencyIndex );
							}
						}
					}
				}
			}
		}
	}
}

ULevelVariantSets::ULevelVariantSets(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	if (!IsTemplate())
	{
		SubscribeToEditorDelegates();
	}
#endif
}


void ULevelVariantSets::BeginDestroy()
{
	Super::BeginDestroy();
#if WITH_EDITOR
	UnsubscribeToEditorDelegates();
	UnsubscribeToDirectorCompiled();
#endif
}

void ULevelVariantSets::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		SubscribeToDirectorCompiled();
	}
#endif
}

void ULevelVariantSets::AddVariantSets(const TArray<UVariantSet*>& NewVariantSets, int32 Index)
{
	Modify();

	if (Index == INDEX_NONE)
	{
		Index = VariantSets.Num();
	}

	TSet<FString> OldNames;
	for (UVariantSet* VarSet : VariantSets)
	{
		OldNames.Add(VarSet->GetDisplayText().ToString());
	}

	// Inserting first ensures we preserve the target order
	VariantSets.Insert(NewVariantSets, Index);

	bool bIsMoveOperation = false;
	TSet<ULevelVariantSets*> ParentsModified;
	for (UVariantSet* NewVarSet : NewVariantSets)
	{
		if (NewVarSet == nullptr)
		{
			continue;
		}

		ULevelVariantSets* OldParent = NewVarSet->GetParent();

		if (OldParent)
		{
			if (OldParent != this)
			{
				OldParent->RemoveVariantSets({NewVarSet});
			}
			else
			{
				bIsMoveOperation = true;
			}
		}

		NewVarSet->Modify();
		NewVarSet->Rename(nullptr, this, REN_DontCreateRedirectors);  // Change parents

		// Update name if we're from a different parent but our names collide
		FString IncomingName = NewVarSet->GetDisplayText().ToString();
		if (OldParent != this && OldNames.Contains(IncomingName))
		{
			NewVarSet->SetDisplayText(FText::FromString(GetUniqueVariantSetName(IncomingName)));
		}
	}

	// If it's a move operation, we'll have to manually clear the old pointers from the array
	if (bIsMoveOperation)
	{
		TSet<UVariantSet*> SetOfNewVariantSets = TSet<UVariantSet*>(NewVariantSets);

		// Sweep back from insertion point nulling old bindings with the same GUID
		for (int32 SweepIndex = Index-1; SweepIndex >= 0; SweepIndex--)
		{
			if (SetOfNewVariantSets.Contains(VariantSets[SweepIndex]))
			{
				VariantSets[SweepIndex] = nullptr;
			}
		}
		// Sweep forward from the end of the inserted segment nulling old bindings with the same GUID
		for (int32 SweepIndex = Index + NewVariantSets.Num(); SweepIndex < VariantSets.Num(); SweepIndex++)
		{
			if (SetOfNewVariantSets.Contains(VariantSets[SweepIndex]))
			{
				VariantSets[SweepIndex] = nullptr;
			}
		}

		// Finally remove null entries
		for (int32 IterIndex = VariantSets.Num() - 1; IterIndex >= 0; IterIndex--)
		{
			if (VariantSets[IterIndex] == nullptr)
			{
				VariantSets.RemoveAt(IterIndex);
			}
		}
	}
}

int32 ULevelVariantSets::GetVariantSetIndex(UVariantSet* VarSet)
{
	if (VarSet == nullptr)
	{
		return INDEX_NONE;
	}

	return VariantSets.Find(VarSet);
}

const TArray<UVariantSet*>& ULevelVariantSets::GetVariantSets() const
{
	return VariantSets;
}

void ULevelVariantSets::RemoveVariantSets(const TArray<UVariantSet*> InVariantSets)
{
	Modify();

	for (UVariantSet* VariantSet : InVariantSets)
	{
		VariantSets.Remove(VariantSet);
		UE::LevelVariantSets::Private::ResetVariantSetDependents( VariantSet );
		VariantSet->Rename(nullptr, GetTransientPackage());
	}
}

FString ULevelVariantSets::GetUniqueVariantSetName(const FString& InPrefix)
{
	TSet<FString> UniqueNames;
	for (UVariantSet* VariantSet : VariantSets)
	{
		UniqueNames.Add(VariantSet->GetDisplayText().ToString());
	}

	if (!UniqueNames.Contains(InPrefix))
	{
		return InPrefix;
	}

	FString VarSetName = FString(InPrefix);

	// Remove potentially existing suffix numbers
	FString LastChar = VarSetName.Right(1);
	while (LastChar.IsNumeric())
	{
		VarSetName.LeftChopInline(1, EAllowShrinking::No);
		LastChar = VarSetName.Right(1);
	}

	// Add a numbered suffix
	if (UniqueNames.Contains(VarSetName) || VarSetName.IsEmpty())
	{
		int32 Suffix = 0;
		while (UniqueNames.Contains(VarSetName + FString::FromInt(Suffix)))
		{
			Suffix += 1;
		}

		VarSetName = VarSetName + FString::FromInt(Suffix);
	}

	return VarSetName;
}

UObject* ULevelVariantSets::GetDirectorInstance(UObject* WorldContext)
{
	if (WorldContext == nullptr || !IsValidChecked(WorldContext) || WorldContext->IsUnreachable() || !WorldContext->GetWorld())
	{
		return nullptr;
	}

	// This will always pick the persistent world, even if WorldContext is in a streamed-in level.
	// This is important as the UWorld that is actually outer to streamed-in levels shouldn't be used generally.
	UWorld* TargetWorld = WorldContext->GetWorld();

	// Check if we already created a director for this world
	TWeakObjectPtr<UObject> FoundDirector = WorldToDirectorInstance.FindRef( TargetWorld );
	if ( FoundDirector.IsValid() )
	{
		return FoundDirector.Get();
	}

	// If not we'll need to create one. It will need to be parented to a LVSActor in that world
	ALevelVariantSetsActor* DirectorOuter = nullptr;

	// Look for a LVSActor in that world that is referencing us
	TArray<AActor*> FoundActors;
	UGameplayStatics::GetAllActorsOfClass(TargetWorld, ALevelVariantSetsActor::StaticClass(), FoundActors);
	for (AActor* Actor : FoundActors)
	{
		// We can never use ALevelVariantSetsActors on streamed-in levels for this, because some functions will fetch the UObject's
		// UWorld by going through ULevel::OwningWorld, and others will just travel up the outer chain, which can leads to different results for streamed levels.
		// Here we ignore those actors by making sure that we only consider the ones directly outer'ed to the persistent world,
		// where this duality won't exist. It may be slightly weird that we will ignore ALevelVariantSetsActors on streamed-in levels that
		// the user may have manually placed, but it should work more consistently this way. These actors don't really have any state anyway,
		// so there is no harm in having an additional actor. Also, see the comment on ULevel::OwningWorld.
		if ( Actor->GetTypedOuter<UWorld>() != TargetWorld )
		{
			continue;
		}

		ALevelVariantSetsActor* ActorAsLVSActor = Cast<ALevelVariantSetsActor>(Actor);
		if (ActorAsLVSActor && ActorAsLVSActor->GetLevelVariantSets() == this)
		{
			DirectorOuter = ActorAsLVSActor;
			break;
		}
	}

	// If we haven't found one, we need to spawn a new LVSActor
	if (DirectorOuter == nullptr)
	{
		FVector Location(0.0f, 0.0f, 0.0f);
		FRotator Rotation(0.0f, 0.0f, 0.0f);
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.OverrideLevel = TargetWorld->PersistentLevel; // If we leave it null it will pick the persistent level *of the streamed in world* instead, which is not actually *the* persistent level
		ALevelVariantSetsActor* NewActor = TargetWorld->SpawnActor<ALevelVariantSetsActor>(Location, Rotation, SpawnInfo);
		NewActor->SetLevelVariantSets(this);
		DirectorOuter = NewActor;
	}

	if (DirectorOuter == nullptr)
	{
		return nullptr;
	}

	ULevelVariantSetsFunctionDirector* TargetDirector = nullptr;
	if ( TObjectPtr<ULevelVariantSetsFunctionDirector>* ExistingDirector = DirectorOuter->DirectorInstances.Find( DirectorClass ) )
	{
		TargetDirector = *ExistingDirector;
	}
	if ( !TargetDirector )
	{
		TargetDirector = NewObject<ULevelVariantSetsFunctionDirector>(DirectorOuter, DirectorClass, NAME_None, RF_Transient);
		DirectorOuter->DirectorInstances.Add( DirectorClass, TargetDirector );
	}

	TargetDirector->GetOnDestroy().AddLambda([this](ULevelVariantSetsFunctionDirector* Director)
	{
		if (this->IsValidLowLevel() && IsValidChecked(this) && !this->IsUnreachable())
		{
			HandleDirectorDestroyed(Director);
		}
	});

	WorldToDirectorInstance.Add( TargetWorld, TargetDirector );
	return TargetDirector;
}

int32 ULevelVariantSets::GetNumVariantSets()
{
	return VariantSets.Num();
}

UVariantSet* ULevelVariantSets::GetVariantSet(int32 VariantSetIndex)
{
	if (VariantSets.IsValidIndex(VariantSetIndex))
	{
		return VariantSets[VariantSetIndex];
	}

	return nullptr;
}

UVariantSet* ULevelVariantSets::GetVariantSetByName(FString VariantSetName)
{
	TObjectPtr<UVariantSet>* VarSetPtr = VariantSets.FindByPredicate([VariantSetName](const UVariantSet* VarSet)
	{
		return VarSet && VarSet->GetDisplayText().ToString() == VariantSetName;
	});

	if (VarSetPtr)
	{
		return *VarSetPtr;
	}
	return nullptr;
}

int32 ULevelVariantSets::GetVariantSetIndex(FString VariantSetName) const
{
	return VariantSets.IndexOfByPredicate([VariantSetName](const UVariantSet* VarSet)
	{
		return VarSet && VarSet->GetDisplayText().ToString() == VariantSetName;
	});
}

#if WITH_EDITOR
void ULevelVariantSets::SetDirectorGeneratedBlueprint(UObject* InDirectorBlueprint)
{
	UBlueprint* InBP = Cast<UBlueprint>(InDirectorBlueprint);
	if (!InBP)
	{
		return;
	}

	DirectorBlueprint = InBP;
	DirectorClass = CastChecked<UBlueprintGeneratedClass>(InBP->GeneratedClass);

	SubscribeToDirectorCompiled();
}

UObject* ULevelVariantSets::GetDirectorGeneratedBlueprint()
{
	return DirectorBlueprint;
}

UBlueprintGeneratedClass* ULevelVariantSets::GetDirectorGeneratedClass()
{
	return DirectorClass;
}

void ULevelVariantSets::OnDirectorBlueprintRecompiled(UBlueprint* InBP)
{
	for (UVariantSet* VarSet : VariantSets)
	{
		for (UVariant* Var : VarSet->GetVariants())
		{
			for (UVariantObjectBinding* Binding : Var->GetBindings())
			{
				Binding->UpdateFunctionCallerNames();
			}
		}
	}
}

UWorld* ULevelVariantSets::GetWorldContext(int32& OutPIEInstanceID)
{
	if (CurrentWorld == nullptr)
	{
		CurrentWorld = ComputeCurrentWorld(CurrentPIEInstanceID);
		check(CurrentWorld);
	}

	OutPIEInstanceID = CurrentPIEInstanceID;
	return CurrentWorld;
}

void ULevelVariantSets::ResetWorldContext()
{
	CurrentWorld = nullptr;
}

void ULevelVariantSets::OnPieEvent(bool bIsSimulating)
{
	ResetWorldContext();
}

void ULevelVariantSets::OnMapChange(uint32 MapChangeFlags)
{
	ResetWorldContext();
}

UWorld* ULevelVariantSets::ComputeCurrentWorld(int32& OutPIEInstanceID)
{
	UWorld* EditorWorld = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		// Return the first PIE world that we can find
		if (Context.WorldType == EWorldType::PIE)
		{
			UWorld* ThisWorld = Context.World();
			if (ThisWorld)
			{
				OutPIEInstanceID = Context.PIEInstance;
				return ThisWorld;
			}
		}
		// Or else return a valid Editor world. For "Standalone mode" the world type is Game.
		// Note that this code won't run in an actual packaged build though, as its inside an #if WITH_EDITOR block
		else if (Context.WorldType == EWorldType::Editor | Context.WorldType == EWorldType::Game)
		{
			EditorWorld = Context.World();
		}
	}

	check(EditorWorld);
	OutPIEInstanceID = INDEX_NONE;
	return EditorWorld;
}

void ULevelVariantSets::SubscribeToEditorDelegates()
{
	FEditorDelegates::MapChange.AddUObject(this, &ULevelVariantSets::OnMapChange);

	// Invalidate CurrentWorld after PIE starts
	FEditorDelegates::PostPIEStarted.AddUObject(this, &ULevelVariantSets::OnPieEvent);

	// This is used as if it was a PostPIEEnded event
	EndPlayDelegateHandle = FGameDelegates::Get().GetEndPlayMapDelegate().AddUObject(this, &ULevelVariantSets::OnMapChange, (uint32)0);
}

void ULevelVariantSets::UnsubscribeToEditorDelegates()
{
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FGameDelegates::Get().GetEndPlayMapDelegate().Remove(EndPlayDelegateHandle);
}

void ULevelVariantSets::SubscribeToDirectorCompiled()
{
	UBlueprint* DirectorBP = Cast<UBlueprint>(DirectorBlueprint);
	if (DirectorBP && IsValidChecked(DirectorBP) && !DirectorBP->IsUnreachable())
	{
		OnBlueprintCompiledHandle = DirectorBP->OnCompiled().AddUObject(this, &ULevelVariantSets::OnDirectorBlueprintRecompiled);
	}
}

void ULevelVariantSets::UnsubscribeToDirectorCompiled()
{
	UBlueprint* DirectorBP = Cast<UBlueprint>(DirectorBlueprint);
	if (DirectorBP && IsValidChecked(DirectorBP) && !DirectorBP->IsUnreachable())
	{
		DirectorBP->OnCompiled().Remove(OnBlueprintCompiledHandle);
	}
}
#endif

void ULevelVariantSets::HandleDirectorDestroyed(ULevelVariantSetsFunctionDirector* Director)
{
	if (!IsValid(Director))
	{
		return;
	}

	for (TMap<UWorld*, TWeakObjectPtr<UObject>>::TIterator Iter(WorldToDirectorInstance); Iter; ++Iter )
	{
		if (Iter->Value.IsValid() && Iter->Value == Director)
		{
			Iter.RemoveCurrent();
		}
	}
}

#undef LOCTEXT_NAMESPACE
