// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSetsActor.h"

#include "LevelVariantSets.h"
#include "VariantSet.h"
#include "Variant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelVariantSetsActor)

ALevelVariantSetsActor::ALevelVariantSetsActor(const FObjectInitializer& Init)
	: Super(Init)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;
}

ULevelVariantSets* ALevelVariantSetsActor::GetLevelVariantSets(bool bLoad)
{
	if (LevelVariantSets.IsValid())
	{
		ULevelVariantSets* VarSets = Cast<ULevelVariantSets>(LevelVariantSets.ResolveObject());
		if (VarSets)
		{
			return VarSets;
		}

		if (bLoad)
		{
			if (IsAsyncLoading())
			{
				LoadPackageAsync(LevelVariantSets.GetLongPackageName());
				return nullptr;
			}
			else
			{
				return Cast<ULevelVariantSets>(LevelVariantSets.TryLoad());
			}
		}
	}

	return nullptr;
}

void ALevelVariantSetsActor::SetLevelVariantSets(ULevelVariantSets* InVariantSets)
{
    LevelVariantSets = InVariantSets;
}

bool ALevelVariantSetsActor::SwitchOnVariantByName(FString VariantSetName, FString VariantName)
{
	ULevelVariantSets* LevelVarSets = GetLevelVariantSets(true);
	if (LevelVarSets)
	{
		for (UVariantSet* VarSet : LevelVarSets->GetVariantSets())
		{
			if (VarSet->GetDisplayText().ToString() == VariantSetName)
			{
				for (UVariant* Var : VarSet->GetVariants())
				{
					if (Var->GetDisplayText().ToString() == VariantName)
					{
						Var->SwitchOn();
						return true;
					}
				}
			}
		}
	}

	return false;
}

bool ALevelVariantSetsActor::SwitchOnVariantByIndex(int32 VariantSetIndex, int32 VariantIndex)
{
	ULevelVariantSets* LevelVarSets = GetLevelVariantSets(true);
	if (LevelVarSets)
	{
		UVariantSet* VarSet = LevelVarSets->GetVariantSet(VariantSetIndex);
		if (VarSet)
		{
			UVariant* Var = VarSet->GetVariant(VariantIndex);
			if (Var)
			{
				Var->SwitchOn();
				return true;
			}
		}
	}

	return false;
}
