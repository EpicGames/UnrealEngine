// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSetsFunctionDirector.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelVariantSetsFunctionDirector)

UWorld* ULevelVariantSetsFunctionDirector::GetWorld() const
{
	return GetTypedOuter<UWorld>();
}
