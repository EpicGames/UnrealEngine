// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyValue.h"

#include "UObject/EnumProperty.h"
#include "VariantManagerContentLog.h"
#include "VariantManagerObjectVersion.h"
#include "VariantObjectBinding.h"

#include "Engine/SCS_Node.h"
#include "UObject/TextProperty.h"
#include "UObject/CoreObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyValue)

#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "PropertyValue"

DEFINE_LOG_CATEGORY(LogVariantContent);

namespace UE
{
	namespace PropertyValue
	{
		namespace Private
		{
			template<typename OldType, typename NewType>
			void ConvertRecordedValueSizes( size_t TargetStructSize, TArray<uint8>& ValueBytes )
			{
				const int32 NumElements = ValueBytes.Num() / sizeof( OldType );
				ensure( ValueBytes.Num() % sizeof( OldType ) == 0 );
				ensure( TargetStructSize == NumElements * sizeof( NewType ) );

				TArray<uint8> ConvertedRecordedData;
				ConvertedRecordedData.SetNumZeroed( TargetStructSize );

				const OldType* OldValues = reinterpret_cast< const OldType* >( ValueBytes.GetData() );
				NewType* NewValues = reinterpret_cast< NewType* >( ConvertedRecordedData.GetData() );

				for ( int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex )
				{
					*NewValues++ = static_cast< NewType >( *OldValues++ );
				}

				ValueBytes = MoveTemp( ConvertedRecordedData );
			}

			size_t GetTargetStructElementSize( UScriptStruct* StructClass )
			{
				if ( StructClass )
				{
					FName StructName = StructClass->GetFName();
					if ( StructName == NAME_Vector )
					{
						return sizeof( FVector::X );
					}
					else if ( StructName == NAME_Rotator )
					{
						return sizeof( FRotator::Pitch );
					}
					else if ( StructName == NAME_Quat )
					{
						return sizeof( FQuat::X );
					}
					else if ( StructName == NAME_Vector4 )
					{
						return sizeof( FVector4::X );
					}
					else if ( StructName == NAME_Vector2D )
					{
						return sizeof( FVector2D::X );
					}
				}

				return 0;
			}

			// The purpose of this function is to upgrade ValueBytes to the correct size/values if we last saved our e.g. FVector
			// recorded data when FVector contained floats, but now it should hold doubles (i.e. ValueBytes holds 12 bytes, but
			// it should really hold 24 now).
			void UpdateRecordedDataSizesIfNeeded( UScriptStruct* StructClass, TArray<uint8>& ValueBytes )
			{
				if ( !StructClass )
				{
					return;
				}

				const size_t TargetStructSize = StructClass->GetCppStructOps()->GetSize();
				const size_t TargetElementSize = GetTargetStructElementSize( StructClass );
				if ( TargetElementSize && TargetStructSize && TargetStructSize != ValueBytes.Num() )
				{
					if ( TargetElementSize == sizeof( double ) )
					{
						ConvertRecordedValueSizes<float, double>( TargetStructSize, ValueBytes );
					}
					else if ( TargetElementSize == sizeof( float ) )
					{
						ConvertRecordedValueSizes<double, float>( TargetStructSize, ValueBytes );
					}
				}
			}
		}
	}
}

// Non-asserting way of checking if an Index is valid for an enum.
// Warning: This will claim that the _MAX entry's index is also invalid
bool IsEnumIndexValid(UEnum* Enum, int32 Index)
{
	if (Enum)
	{
		FName EntryName = Enum->GetNameByIndex(Index);

		bool bIsInvalidEntry = (EntryName == NAME_None);
		bool bIndexIsMAXEntry = Enum->ContainsExistingMax() && (EntryName.ToString().EndsWith(TEXT("_MAX"), ESearchCase::CaseSensitive));

		return (!bIsInvalidEntry && !bIndexIsMAXEntry);
	}

	return false;
}

// Assuming that ValueBytes stores the bytes of a value for Enum, this will try to convert it into
// the index of that value. May return INDEX_NONE if it doesn't match or if it is invalid in some other way.
// Warning: This will happily return the index of the _MAX enum entry if that is the case.
int32 EnumValueBytesToIndex(UEnum* Enum, const TArray<uint8>& ValueBytes, bool bEnumIsSignedInt)
{
	const uint8* DataPointer = ValueBytes.GetData();
	int64 EnumValue = 0;
	int32 Size = ValueBytes.Num();

	if (bEnumIsSignedInt)
	{
		switch (Size)
		{
		case sizeof(int8):
		{
			int8 CastValue = *(int8*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(int16):
		{
			int16 CastValue = *(int16*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(int32):
		{
			int32 CastValue = *(int32*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(int64):
		{
			int64 CastValue = *(int64*)DataPointer;
			EnumValue = CastValue;
			break;
		}
		default:
			UE_LOGF(LogVariantContent, Error, "Invalid size for signed int value: %d", Size);
			return INDEX_NONE;
		}
	}
	else
	{
		switch (Size)
		{
		case sizeof(uint8):
		{
			uint8 CastValue = *(uint8*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(uint16):
		{
			uint16 CastValue = *(uint16*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(uint32):
		{
			uint32 CastValue = *(uint32*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		case sizeof(uint64):
		{
			uint64 CastValue = *(int64*)DataPointer;
			EnumValue = static_cast<int64>(CastValue);
			break;
		}
		default:
			UE_LOGF(LogVariantContent, Error, "Invalid size for unsigned int value: %d", Size);
			return INDEX_NONE;
		}
	}

	return Enum->GetIndexByValue(EnumValue);
}

// Searches the value of Index within Enum and return an array of bytes that stores that value
TArray<uint8> EnumIndexToValueBytes(UEnum* Enum, int32 Index, int32 Size, bool bEnumIsSignedInt)
{
	TArray<uint8> NewValueBytes;
	NewValueBytes.SetNumZeroed(Size);

	// Do this check or else GetValueByIndex may assert
	if (!IsEnumIndexValid(Enum, Index))
	{
		return NewValueBytes;
	}

	int64 EnumVal = Enum->GetValueByIndex(Index);

	// We have to do this below because GetValueByIndex always gives us an int64. We need to cast this down to the
	// ElementSize of the property

	if (bEnumIsSignedInt)
	{
		switch (Size)
		{
		case sizeof(int8):
		{
			int8 CastValue = static_cast<int8>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(int16):
		{
			int16 CastValue = static_cast<int16>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(int32):
		{
			int32 CastValue = static_cast<int32>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(int64):
		{
			int64 CastValue = static_cast<int64>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		default:
			UE_LOGF(LogVariantContent, Error, "Invalid size for signed int value: %d", Size);
			return NewValueBytes;
		}
	}
	else
	{
		switch (Size)
		{
		case sizeof(uint8):
		{
			uint8 CastValue = static_cast<uint8>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(uint16):
		{
			uint16 CastValue = static_cast<uint16>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(uint32):
		{
			uint32 CastValue = static_cast<uint32>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		case sizeof(uint64):
		{
			uint64 CastValue = static_cast<uint64>(EnumVal);
			FMemory::Memcpy(NewValueBytes.GetData(), (uint8*)(&CastValue), Size);
			break;
		}
		default:
			UE_LOGF(LogVariantContent, Error, "Invalid size for unsigned int value: %d", Size);
			return NewValueBytes;
		}
	}

	return NewValueBytes;
}

UPropertyValue::UPropertyValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LeafPropertyClass(nullptr)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorDelegates::EndPIE.AddUObject(this, &UPropertyValue::OnPIEEnded);
	}
#endif
}

void UPropertyValue::Init(const TArray<FCapturedPropSegment>& InCapturedPropSegments, FFieldClass* InLeafPropertyClass, const FString& InFullDisplayString, const FName& InPropertySetterName, EPropertyValueCategory InCategory)
{
	CapturedPropSegments = InCapturedPropSegments;
	LeafPropertyClass = InLeafPropertyClass;
	FullDisplayString = InFullDisplayString;
	PropertySetterName = InPropertySetterName;
	PropCategory = InCategory;

	if (PropertySetterName == TEXT("SetRelativeLocation") || PropertySetterName == TEXT("SetRelativeRotation"))
	{
		PropertySetterName = FName(*(TEXT("K2_") + PropertySetterName.ToString()));
	}

	ClearLastResolve();
	ValueBytes.Empty();
	DefaultValue.Empty();
	TempObjPtr.Reset();
}

void UPropertyValue::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif
}

#if WITH_EDITOR

void UPropertyValue::OnPIEEnded(const bool bIsSimulatingInEditor)
{
	ClearLastResolve();
}

#endif

UVariantObjectBinding* UPropertyValue::GetParent() const
{
	return Cast<UVariantObjectBinding>(GetOuter());
}

uint32 UPropertyValue::GetPropertyPathHash()
{
	uint32 Hash = 0;
	for (const FCapturedPropSegment& Seg : CapturedPropSegments)
	{
		Hash = HashCombine(Hash, GetTypeHash(Seg.PropertyName));
		Hash = HashCombine(Hash, GetTypeHash(Seg.PropertyIndex));
		Hash = HashCombine(Hash, GetTypeHash(Seg.ComponentName));
	}
	return Hash;
}

void UPropertyValue::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FVariantManagerObjectVersion::GUID);
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	int32 CustomVersion = Ar.CustomVer(FVariantManagerObjectVersion::GUID);

	if (Ar.CustomVer(FCoreObjectVersion::GUID) >= FCoreObjectVersion::FProperties)
	{
		Ar << LeafPropertyClass;
	}
	else if (Ar.IsLoading() && LeafPropertyClass_DEPRECATED)
	{
		LeafPropertyClass = FFieldClass::GetNameToFieldClassMap().FindRef(LeafPropertyClass_DEPRECATED->GetFName());
	}

	if (Ar.IsSaving())
	{
		Ar << TempObjPtr;

		if (CustomVersion >= FVariantManagerObjectVersion::CorrectSerializationOfFStringBytes)
		{
			// These are either setup during IsLoading() or when SetRecordedData
			Ar << TempName;
			Ar << TempStr;
			Ar << TempText;
		}
		else if (CustomVersion >= FVariantManagerObjectVersion::CorrectSerializationOfFNameBytes)
		{
			FName Name;
			if (FFieldClass* PropClass = GetPropertyClass())
			{
				if (PropClass->IsChildOf(FNameProperty::StaticClass()))
				{
					Name = *((FName*)ValueBytes.GetData());
				}
			}

			Ar << Name;
		}
	}
	else if (Ar.IsLoading())
	{
		DefaultValue.Empty();

		if (PropertySetterName == TEXT("SetRelativeLocation") || PropertySetterName == TEXT("SetRelativeRotation"))
		{
			PropertySetterName = FName(*(TEXT("K2_") + PropertySetterName.ToString()));
		}

		Ar << TempObjPtr;

		// Before this version, properties were stored an array of FProperty*. Convert them to
		// CapturedPropSegment and clear the deprecated arrays
		if (CustomVersion < FVariantManagerObjectVersion::SerializePropertiesAsNames)
		{
			UE_LOGF(LogVariantContent, Warning, "Captured property '%ls' was created with an older Unreal Studio version (4.21 or less). A conversion to the new storage format is required and will be attempted. There may be some data loss.", *FullDisplayString);

			int32 NumDeprecatedProps = Properties_DEPRECATED.Num();
			if (NumDeprecatedProps > 0)
			{
				// Back then we didn't store the class directly, and just fetched it from the leaf-most property
				// Try to do that again as it might help decode ValueBytes if those properties were string types
				TFieldPath<FProperty> LastProp = Properties_DEPRECATED.Last();
				LeafPropertyClass = FFieldClass::GetNameToFieldClassMap().FindRef(LastProp->GetClass()->GetFName());

				CapturedPropSegments.Reserve(NumDeprecatedProps);
				int32 Index = 0;
				for (Index = 0; Index < NumDeprecatedProps; Index++)
				{
					TFieldPath<FProperty> Prop = Properties_DEPRECATED[Index];
					if (Prop == nullptr || !Prop->IsValidLowLevel() || !PropertyIndices_DEPRECATED.IsValidIndex(Index))
					{
						break;
					}

					FCapturedPropSegment* NewSeg = new(CapturedPropSegments) FCapturedPropSegment;
					NewSeg->PropertyName = Prop->GetName();
					NewSeg->PropertyIndex = PropertyIndices_DEPRECATED[Index];
				}

				// Conversion succeeded
				if (Index == NumDeprecatedProps)
				{
					Properties_DEPRECATED.Reset();
					PropertyIndices_DEPRECATED.Reset();
				}
				else
				{
					UE_LOGF(LogVariantContent, Warning, "Failed to convert property '%ls'! Captured data will be ignored and property will fail to resolve.", *FullDisplayString);
					CapturedPropSegments.Reset();
				}
			}
		}

		if (CustomVersion >= FVariantManagerObjectVersion::CorrectSerializationOfFStringBytes)
		{
			Ar << TempName;
			Ar << TempStr;
			Ar << TempText;

			if (FFieldClass* PropClass = GetPropertyClass())
			{
				if (PropClass->IsChildOf(FNameProperty::StaticClass()))
				{
					SetRecordedDataInternal((uint8*)&TempName, sizeof(FName));
				}
				else if (PropClass->IsChildOf(FStrProperty::StaticClass()))
				{
					SetRecordedDataInternal((uint8*)&TempStr, sizeof(FString));
				}
				else if (PropClass->IsChildOf(FTextProperty::StaticClass()))
				{
					SetRecordedDataInternal((uint8*)&TempText, sizeof(FText));
				}
			}
			else
			{
				//UE_LOGF(LogVariantContent, Error, "Failed to retrieve property class for property '%ls'", *GetFullDisplayString());
				bHasRecordedData = false;
			}
		}
		else if (CustomVersion >= FVariantManagerObjectVersion::CorrectSerializationOfFNameBytes)
		{
			FName Name;
			Ar << Name;

			if (FFieldClass* PropClass = GetPropertyClass())
			{
				if (PropClass == FNameProperty::StaticClass())
				{
					SetRecordedDataInternal((uint8*)&Name, sizeof(FName));
				}
			}
			else
			{
				//UE_LOGF(LogVariantContent, Error, "Failed to retrieve property class for property '%ls'", *GetFullDisplayString());
				bHasRecordedData = false;
			}
		}

		if (UEnum* Enum = GetEnumPropertyEnum())
		{
			SanitizeRecordedEnumData();
		}
	}
}

bool UPropertyValue::Resolve(UObject* Object)
{
	if (Object == nullptr)
	{
		UVariantObjectBinding* Parent = GetParent();
		if (Parent)
		{
			Object = Parent->GetObject();
		}
	}

	if (Object == nullptr)
	{
		return false;
	}

	if (CapturedPropSegments.Num() == 0)
	{
		return false;
	}

	const bool bStartedUnresolved = !HasValidResolve();

	ParentContainerObject = Object;
	if (!ResolvePropertiesRecursive(Object->GetClass(), Object, 0))
	{
		return false;
	}

	// Try to recover if we had a project that didn't have the LeafPropertyClass fix, so that
	// we don't lose all our variants
	if (LeafPropertyClass == nullptr && LeafProperty != nullptr)
	{
		LeafPropertyClass = LeafProperty->GetClass();
	}

	if (ParentContainerClass)
	{
		if (const UClass* Class = Cast<const UClass>(ParentContainerClass))
		{
			PropertySetter = Class->FindFunctionByName(PropertySetterName);
			if (PropertySetter)
			{
				FFieldClass* ThisClass = GetPropertyClass();
				bool bFoundParameterWithClassType = false;

				for (TFieldIterator<FProperty> It(PropertySetter); It; ++It)
				{
					FProperty* Prop = *It;

					if (ThisClass == Prop->GetClass())
					{
						bFoundParameterWithClassType = true;
						//break;
					}
				}

				if (!bFoundParameterWithClassType)
				{
					UE_LOGF(LogVariantContent, Error, "Property setter does not have a parameter that can receive an object of the property type (%ls)!", *ThisClass->GetName());
					PropertySetter = nullptr;
				}
			}
		}
	}

	if ( bStartedUnresolved && HasValidResolve() && bHasRecordedData )
	{
		// We can only do this after we resolve because we need to know the struct property's struct class, which also
		// means we can't do it on Serialize()
		UE::PropertyValue::Private::UpdateRecordedDataSizesIfNeeded( GetStructPropertyStruct(), ValueBytes );
	}

	return true;
}

bool UPropertyValue::HasValidResolve() const
{
	if (ParentContainerAddress == nullptr || ParentContainerClass == nullptr)
	{
		return false;
	}

	// We might be pointing at a component that was created via a construction script.
	// Whenever an actor's property changes, all of its components that were generated
	// by construction scripts will be destroyed and regenerated, so we will need to re-resolve
	// We can't check the resolve state this way for USTRUCTs however (like FPostProcessSettings)
	if (!ParentContainerClass->IsA(UScriptStruct::StaticClass()))
	{
		if (UObject* Container = (UObject*)ParentContainerAddress)
		{
			// Replicate IsValidLowLevel without the warning, plus a couple of extra
			// checks because this random address might be pointing at a destroyed component
			if (!GUObjectArray.IsValid(this) ||
				!Container->GetClass() ||
				!IsValidChecked(Container) ||
				Container->IsUnreachable() ||
				Container->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
			{
				return false;
			}
		}
	}

	return true;
}

void UPropertyValue::ClearLastResolve()
{
	LeafProperty = nullptr;
	ParentContainerClass = nullptr;
	ParentContainerAddress = nullptr;
	PropertyValuePtr = nullptr;
}

void* UPropertyValue::GetPropertyParentContainerAddress() const
{
	return ParentContainerAddress;
}

UStruct* UPropertyValue::GetPropertyParentContainerClass() const
{
	return ParentContainerClass;
}

TArray<uint8> UPropertyValue::GetDataFromResolvedObject() const
{
	int32 PropertySizeBytes = GetValueSizeInBytes();
	TArray<uint8> CurrentData;
	CurrentData.SetNumZeroed(PropertySizeBytes);

	if (!HasValidResolve())
	{
		return CurrentData;
	}

	if (FBoolProperty* PropAsBool = CastField<FBoolProperty>(LeafProperty))
	{
		bool* BytesAddress = (bool*)CurrentData.GetData();
		*BytesAddress = PropAsBool->GetPropertyValue(PropertyValuePtr);
	}
	else
	{
		FMemory::Memcpy(CurrentData.GetData(), PropertyValuePtr, PropertySizeBytes);
	}

	return CurrentData;
}

void UPropertyValue::RecordDataFromResolvedObject()
{
	if (!Resolve())
	{
		return;
	}

	TArray<uint8> NewData = GetDataFromResolvedObject();
	SetRecordedData(NewData.GetData(), NewData.Num());

	// If we don't have parameter defaults, try fetching them
#if WITH_EDITOR
	if (PropertySetter && PropertySetterParameterDefaults.Num() == 0)
	{
		for (TFieldIterator<FProperty> It(PropertySetter); It; ++It)
		{
			FProperty* Prop = *It;
			FString Defaults;

			// Store property setter parameter defaults, as this is kept in metadata which is not available at runtime
			UEdGraphSchema_K2::FindFunctionParameterDefaultValue(PropertySetter, Prop, Defaults);

			if (!Defaults.IsEmpty())
			{
				PropertySetterParameterDefaults.Add(Prop->GetName(), Defaults);
			}
		}
	}
#endif //WITH_EDITOR

	OnPropertyRecorded.Broadcast();
}

void UPropertyValue::ApplyDataToResolvedObject()
{
	if (!HasRecordedData() || !Resolve())
	{
		return;
	}

	// Modify owner actor
	UObject* ContainerOwnerObject = nullptr;
	if (UVariantObjectBinding* Parent = GetParent())
	{
		if (UObject* OwnerActor = Parent->GetObject())
		{
			OwnerActor->SetFlags(RF_Transactional);
			OwnerActor->Modify();
			ContainerOwnerObject = OwnerActor;
		}
	}

	// Modify container component
	UObject* ContainerObject = nullptr;
	if (ParentContainerObject && ParentContainerObject->IsA(UActorComponent::StaticClass()))
	{
		ParentContainerObject->SetFlags(RF_Transactional);
		ParentContainerObject->Modify();
		ContainerObject = ParentContainerObject;
	}

	if (PropertySetter)
	{
		// If we resolved, this is valid
		UObject* TargetObject = ( UObject* ) ParentContainerAddress;

		// For UE-121592: Set relative rotation using SetRelativeRotationExact because if the rotation we want to set
		// contains a >90 degree Y value, the value shown on the details panel after applying will be the equivalent rotation
		// where Y value is <90, but that also has X and Y rotations (e.g. [0, 95, 0] will become [180, 85, 180]).
		// This is not so bad on the details panel, but if the sequencer is involved and we want to animate from [0, 89, 0] to
		// [0, 95, 0], we may actually end up interpolating between [0, 89, 0] to [180, 85, 180], which will not look correct
		if ( PropertySetter->GetName() == TEXT( "K2_SetRelativeRotation" ) )
		{
			if ( const FRotator* RecordedRotator = reinterpret_cast<const FRotator*>( GetRecordedData().GetData() ) )
			{
				if ( USceneComponent* Comp = Cast<USceneComponent>( TargetObject ) )
				{
					Comp->SetRelativeRotationExact( *RecordedRotator );
				}
			}
		}
		else
		{
			ApplyViaFunctionSetter( TargetObject );
		}
	}
	// Bool properties need to be set in a particular way since they hold internal private
	// masks and offsets
	else if (FBoolProperty* PropAsBool = CastField<FBoolProperty>(LeafProperty))
	{
		bool* ValueBytesAsBool = (bool*)ValueBytes.GetData();
		PropAsBool->SetPropertyValue(PropertyValuePtr, *ValueBytesAsBool);
	}
	else if (FEnumProperty* PropAsEnum = CastField<FEnumProperty>(LeafProperty))
	{
		FNumericProperty* UnderlyingProp = PropAsEnum->GetUnderlyingProperty();
		int32 PropertySizeBytes = UnderlyingProp->GetElementSize();

		ValueBytes.SetNum(PropertySizeBytes);
		FMemory::Memcpy(PropertyValuePtr, ValueBytes.GetData(), PropertySizeBytes);
	}
	else if (FNameProperty* PropAsName = CastField<FNameProperty>(LeafProperty))
	{
		FName Value = GetNamePropertyName();
		PropAsName->SetPropertyValue(PropertyValuePtr, Value);
	}
	else if (FStrProperty* PropAsStr = CastField<FStrProperty>(LeafProperty))
	{
		FString Value = GetStrPropertyString();
		PropAsStr->SetPropertyValue(PropertyValuePtr, Value);
	}
	else if (FTextProperty* PropAsText = CastField<FTextProperty>(LeafProperty))
	{
		FText Value = GetTextPropertyText();
		PropAsText->SetPropertyValue(PropertyValuePtr, Value);
	}
	else
	{
		// Never access ValueBytes directly as we might need to fixup FObjectProperty values
		const TArray<uint8>& RecordedData = GetRecordedData();
		FMemory::Memcpy(PropertyValuePtr, RecordedData.GetData(), GetValueSizeInBytes());
	}

#if WITH_EDITOR
	if (ContainerObject)
	{
		ContainerObject->PostEditChange();
	}
	if (ContainerOwnerObject && ContainerOwnerObject != ContainerObject)
	{
		ContainerOwnerObject->PostEditChange();
	}
#endif
	OnPropertyApplied.Broadcast();
}

FFieldClass* UPropertyValue::GetPropertyClass() const
{
	return LeafPropertyClass;
}

EPropertyValueCategory UPropertyValue::GetPropCategory() const
{
	return PropCategory;
}

UScriptStruct* UPropertyValue::GetStructPropertyStruct() const
{
	FProperty* Prop = GetProperty();
	if (FStructProperty* StructProp = CastField<FStructProperty>(Prop))
	{
		return StructProp->Struct;
	}

	return nullptr;
}

UClass* UPropertyValue::GetObjectPropertyObjectClass() const
{
	if (FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(GetProperty()))
	{
		return ObjProp->PropertyClass;
	}

	return nullptr;
}

UEnum* UPropertyValue::GetEnumPropertyEnum() const
{
	FProperty* Property = GetProperty();
	if (FEnumProperty* EnumProp = CastField<FEnumProperty>(Property))
	{
		return EnumProp->GetEnum();
	}
	else if (FNumericProperty* NumProp = CastField<FNumericProperty>(Property))
	{
		return NumProp->GetIntPropertyEnum();
	}

	return nullptr;
}

bool UPropertyValue::ContainsProperty(const FProperty* Prop) const
{
	return LeafProperty && LeafProperty == Prop;
}

const TArray<FCapturedPropSegment>& UPropertyValue::GetCapturedPropSegments() const
{
	return CapturedPropSegments;
}

// @Copypaste from PropertyEditorHelpers
TArray<FName> UPropertyValue::GetValidEnumsFromPropertyOverride()
{
	UEnum* Enum = GetEnumPropertyEnum();
	if (Enum == nullptr)
	{
		return TArray<FName>();
	}

	TArray<FName> ValidEnumValues;

#if WITH_EDITOR
	static const FName ValidEnumValuesName("ValidEnumValues");
	if(LeafProperty->HasMetaData(ValidEnumValuesName))
	{
		TArray<FString> ValidEnumValuesAsString;

		LeafProperty->GetMetaData(ValidEnumValuesName).ParseIntoArray(ValidEnumValuesAsString, TEXT(","));
		for(auto& Value : ValidEnumValuesAsString)
		{
			Value.TrimStartInline();
			ValidEnumValues.Add(*Enum->GenerateFullEnumName(*Value));
		}
	}
#endif

	return ValidEnumValues;
}

// @Copypaste from PropertyEditorHelpers
FString UPropertyValue::GetEnumDocumentationLink()
{
#if WITH_EDITOR
	if(LeafProperty != nullptr)
	{
		const FByteProperty* ByteProperty = CastField<FByteProperty>(LeafProperty);
		const FEnumProperty* EnumProperty = CastField<FEnumProperty>(LeafProperty);
		if(ByteProperty || EnumProperty || (LeafProperty->IsA(FStrProperty::StaticClass()) && LeafProperty->HasMetaData(TEXT("Enum"))))
		{
			UEnum* Enum = nullptr;
			if(ByteProperty)
			{
				Enum = ByteProperty->Enum;
			}
			else if (EnumProperty)
			{
				Enum = EnumProperty->GetEnum();
			}
			else
			{

				const FString& EnumName = LeafProperty->GetMetaData(TEXT("Enum"));
				Enum = UClass::TryFindTypeSlow<UEnum>(EnumName, EFindFirstObjectOptions::ExactClass);
			}

			if(Enum)
			{
				return FString::Printf(TEXT("Shared/Enums/%s"), *Enum->GetName());
			}
		}
	}
#endif

	return TEXT("");
}

int32 UPropertyValue::GetRecordedDataAsEnumIndex()
{
	UEnum* Enum = GetEnumPropertyEnum();
	if (!Enum)
	{
		UE_LOGF(LogVariantContent, Error, "Invalid enum for enum property '%ls'", *GetFullDisplayString());
		return INDEX_NONE;
	}

	if (!HasRecordedData())
	{
		UE_LOGF(LogVariantContent, Error, "Enum property '%ls' has no recorded data!", *GetFullDisplayString());
		return INDEX_NONE;
	}

	const TArray<uint8>& RecordedData = GetRecordedData();
	ensure(RecordedData.Num() == GetValueSizeInBytes());

	bool bEnumIsSignedInt = IsNumericPropertySigned();
	if (!bEnumIsSignedInt && !IsNumericPropertyUnsigned())
	{
		UE_LOGF(LogVariantContent, Error, "Invalid underlying format for enum property!");
		return INDEX_NONE;
	}

	return EnumValueBytesToIndex(Enum, RecordedData, bEnumIsSignedInt);
}

void UPropertyValue::SanitizeRecordedEnumData()
{
	UEnum* Enum = GetEnumPropertyEnum();
	if (!Enum)
	{
		UE_LOGF(LogVariantContent, Error, "Invalid enum for enum property '%ls'", *GetFullDisplayString());
		return;
	}

	int32 CurrentIndex = GetRecordedDataAsEnumIndex();
	if (!IsEnumIndexValid(Enum, CurrentIndex))
	{
		if (IsEnumIndexValid(Enum, 0))
		{
			// All the enums we can capture are UENUMs, which are guaranteed to have at least one entry
			SetRecordedDataFromEnumIndex(0);
		}
		else
		{
			UE_LOGF(LogVariantContent, Error, "Enum property '%ls' points to an enum that doesn't have any valid values!", *GetFullDisplayString());
			return;
		}
	}
}

void UPropertyValue::SetRecordedDataFromEnumIndex(int32 Index)
{
	UEnum* Enum = GetEnumPropertyEnum();
	if (!Enum)
	{
		UE_LOGF(LogVariantContent, Error, "Invalid enum for enum property '%ls'", *GetFullDisplayString());
		return;
	}

	if(!IsEnumIndexValid(Enum, Index))
	{
		UE_LOGF(LogVariantContent, Error, "Invalid index for enum '%ls'!", *Enum->GetName());
		return;
	}

	bool bEnumIsSignedInt = IsNumericPropertySigned();
	if (!bEnumIsSignedInt && !IsNumericPropertyUnsigned())
	{
		UE_LOGF(LogVariantContent, Error, "Invalid underlying format for enum property!");
		return;
	}

	int32 Size = GetValueSizeInBytes();
	TArray<uint8> NewValueBytes = EnumIndexToValueBytes(Enum, Index, Size, bEnumIsSignedInt);

	ensure(NewValueBytes.Num() == Size);

	Modify();

	SetRecordedDataInternal(NewValueBytes.GetData(), Size);
}

bool UPropertyValue::IsNumericPropertySigned()
{
	FProperty* Prop = GetProperty();
	if (FNumericProperty* NumericProp = CastField<FNumericProperty>(Prop))
	{
		return NumericProp->IsInteger() && NumericProp->CanHoldValue(-1);
	}
	else if (FEnumProperty* EnumProp = CastField<FEnumProperty>(Prop))
	{
		NumericProp = EnumProp->GetUnderlyingProperty();
		return NumericProp->IsInteger() && NumericProp->CanHoldValue(-1);
	}

	return false;
}

bool UPropertyValue::IsNumericPropertyUnsigned()
{
	FProperty* Prop = GetProperty();
	if (FNumericProperty* NumericProp = CastField<FNumericProperty>(Prop))
	{
		return NumericProp->IsInteger() && !NumericProp->CanHoldValue(-1);
	}
	else if (FEnumProperty* EnumProp = CastField<FEnumProperty>(Prop))
	{
		NumericProp = EnumProp->GetUnderlyingProperty();
		return NumericProp->IsInteger() && !NumericProp->CanHoldValue(-1);
	}

	return false;
}

bool UPropertyValue::IsNumericPropertyFloatingPoint()
{
	FProperty* Prop = GetProperty();
	if (FNumericProperty* NumericProp = CastField<FNumericProperty>(Prop))
	{
		return NumericProp->IsFloatingPoint();
	}
	else if (FEnumProperty* EnumProp = CastField<FEnumProperty>(Prop))
	{
		NumericProp = EnumProp->GetUnderlyingProperty();
		return NumericProp->IsFloatingPoint();
	}

	return false;
}

const FName& UPropertyValue::GetNamePropertyName() const
{
	return TempName;
}

const FString& UPropertyValue::GetStrPropertyString() const
{
	return TempStr;
}

const FText& UPropertyValue::GetTextPropertyText() const
{
	return TempText;
}

FName UPropertyValue::GetPropertyName() const
{
	FProperty* Prop = GetProperty();
	if (Prop)
	{
		return Prop->GetFName();
	}

	return FName();
}

FText UPropertyValue::GetPropertyTooltip() const
{
#if WITH_EDITOR
	FProperty* Prop = GetProperty();
	if (Prop)
	{
		return Prop->GetToolTipText();
	}
#endif

	return FText();
}

const FString& UPropertyValue::GetFullDisplayString() const
{
	return FullDisplayString;
}

FString UPropertyValue::GetLeafDisplayString() const
{
	FString LeftString;
	FString RightString;

	if(FullDisplayString.Split(PATH_DELIMITER, &LeftString, &RightString, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		return RightString;
	}

	return FullDisplayString;
}

FString UPropertyValue::GetBaseDisplayString() const
{
	FString LeftString;
	FString RightString;

	if(FullDisplayString.Split(PATH_DELIMITER, &LeftString, &RightString, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		return LeftString;
	}

	return FullDisplayString;
}

int32 UPropertyValue::GetValueSizeInBytes() const
{
	FProperty* Prop = GetProperty();
	if (FEnumProperty* PropAsEnumProp = CastField<FEnumProperty>(Prop))
	{
		return PropAsEnumProp->GetUnderlyingProperty()->GetElementSize();
	}
	else if (Prop)
	{
		return Prop->GetElementSize();
	}

	UE_LOGF(LogVariantContent, Warning, "Returning size zero for PropertyValue '%ls'", *GetFullDisplayString());
	return 0;
}

int32 UPropertyValue::GetPropertyOffsetInBytes() const
{
	FProperty* Prop = GetProperty();
	if (Prop)
	{
		return Prop->GetOffset_ForInternal();
	}

	// Dangerous
	UE_LOGF(LogVariantContent, Warning, "Returning offset zero for PropertyValue '%ls'", *GetFullDisplayString());
	return 0;
}

bool UPropertyValue::HasRecordedData() const
{
	return bHasRecordedData;
}

const TArray<uint8>& UPropertyValue::GetRecordedData()
{
	ValueBytes.SetNum(GetValueSizeInBytes());

	// If we're holding an UObject* we always need to go through our TempObjPtr first because the UObject may have
	// been collected and the address reused since the last time we loaded it
	FFieldClass* PropClass = GetPropertyClass();
	if (bHasRecordedData && PropClass && PropClass->IsChildOf(FObjectPropertyBase::StaticClass()) && !TempObjPtr.IsNull())
	{
		UObject* Obj = TempObjPtr.LoadSynchronous();
		if ( Obj && Obj->IsValidLowLevel() && Obj->IsA( GetObjectPropertyObjectClass() ) )
		{
			SetRecordedDataInternal((uint8*)&Obj, sizeof(UObject*));
		}
		else
		{
			UE_LOGF(LogVariantContent, Warning, "Failed to find object with the correct class at path '%ls' for property '%ls'. Value will be ignored.",
				*TempObjPtr.ToString(),
				*FullDisplayString
			);

			// Reset our storage to nullptr
			ValueBytes.SetNumUninitialized( GetValueSizeInBytes() );
			FMemory::Memset( ValueBytes.GetData(), 0, ValueBytes.Num() );
			bHasRecordedData = false;
		}
	}

	return ValueBytes;
}

void UPropertyValue::SetRecordedData(const uint8* NewDataBytes, int32 NumBytes, int32 Offset)
{
	Modify();

	if (NumBytes > 0)
	{
		// Because the string types are all handles into arrays/data, we need to reinterpret NewDataBytes
		// first, then copy that object into our Temps and have our ValueBytes refer to it instead.
		// This ensures we own the FString that we're pointing at (and so its internal data array)
		if (NumBytes == sizeof(FName) && GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
		{
			TempName = *((FName*)NewDataBytes);
			SetRecordedDataInternal((uint8*)&TempName, NumBytes, Offset);
		}
		else if (NumBytes == sizeof(FString) && GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()))
		{
			TempStr = *((FString*)NewDataBytes);
			SetRecordedDataInternal((uint8*)&TempStr, NumBytes, Offset);
		}
		else if (NumBytes == sizeof(FText) && GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
		{
			TempText = *((FText*)NewDataBytes);
			SetRecordedDataInternal((uint8*)&TempText, NumBytes, Offset);
		}
		else
		{
			SetRecordedDataInternal(NewDataBytes, NumBytes, Offset);

			// Keep TempObjPtr up-to-date because we'll always first fetch UObject references from there first
			// when getting recorded data, as we have no guarantee that the UObject our ValueBytes wasn't collected
			// from under us
			FFieldClass* PropClass = GetPropertyClass();
			if (PropClass && PropClass->IsChildOf(FObjectPropertyBase::StaticClass()))
			{
				UObject* Obj = *( ( UObject** ) ValueBytes.GetData() );
				TempObjPtr = Obj;
			}

			if (UEnum* Enum = GetEnumPropertyEnum())
			{
				SanitizeRecordedEnumData();
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
uint32 UPropertyValue::GetDisplayOrder() const
{
	return DisplayOrder;
}

void UPropertyValue::SetDisplayOrder(uint32 InDisplayOrder)
{
	if (InDisplayOrder == DisplayOrder)
	{
		return;
	}

	Modify();

	DisplayOrder = InDisplayOrder;
}
#endif

void UPropertyValue::SetRecordedDataInternal(const uint8* NewDataBytes, int32 NumBytes, int32 Offset)
{
	if (NumBytes <= 0)
	{
		return;
	}

	if (ValueBytes.Num() < NumBytes + Offset)
	{
		ValueBytes.SetNumUninitialized(NumBytes+Offset);
	}

	FMemory::Memcpy(ValueBytes.GetData() + Offset, NewDataBytes, NumBytes);
	bHasRecordedData = true;
}

const TArray<uint8>& UPropertyValue::GetDefaultValue()
{
	if (DefaultValue.Num() == 0)
	{
		if (UVariantObjectBinding* Binding = GetParent())
		{
			if (UObject* Object = Binding->GetObject())
			{
				int32 NumBytes = GetValueSizeInBytes();
				DefaultValue.SetNumZeroed(NumBytes);

				if (Resolve(Object->GetClass()->GetDefaultObject()))
				{
					if (FBoolProperty* PropAsBool = CastField<FBoolProperty>(LeafProperty))
					{
						bool* DefaultValueAsBoolPtr = (bool*)DefaultValue.GetData();
						*DefaultValueAsBoolPtr = PropAsBool->GetPropertyValue(PropertyValuePtr);
					}
					else
					{
						if (FEnumProperty* PropAsEnum = CastField<FEnumProperty>(LeafProperty))
						{
							FNumericProperty* UnderlyingProp = PropAsEnum->GetUnderlyingProperty();
							NumBytes = UnderlyingProp->GetElementSize();
						}

						// If we're a material property value we won't have PropertyValuePtr
						if (PropertyValuePtr)
						{
							if (FSoftObjectProperty* PropAsSoft = CastField<FSoftObjectProperty>(LeafProperty))
							{
								UObject* DefaultObj = PropAsSoft->LoadObjectPropertyValue(PropertyValuePtr);
								FMemory::Memcpy(DefaultValue.GetData(), (uint8*)&DefaultObj, NumBytes);
							}
							else
							{
							FMemory::Memcpy(DefaultValue.GetData(), PropertyValuePtr, NumBytes);
						}

						}

						// If a valid enum value hasn't been specified as default, the default will be the _MAX value
						// which we don't allow recording. This will detect that and change the value to match the
						// value of whatever enum is index zero
						if (UEnum* Enum = GetEnumPropertyEnum())
						{
							bool bEnumIsSignedInt = IsNumericPropertySigned();
							if (!bEnumIsSignedInt && !IsNumericPropertyUnsigned())
							{
								UE_LOGF(LogVariantContent, Error, "Invalid underlying format for enum property!");
								return DefaultValue;
							}

							int32 Size = GetValueSizeInBytes();
							int32 Index = EnumValueBytesToIndex(Enum, DefaultValue, bEnumIsSignedInt);
							if (!IsEnumIndexValid(Enum, Index))
							{
								if (IsEnumIndexValid(Enum, 0))
								{
									TArray<uint8> SanitizedDefault = EnumIndexToValueBytes(Enum, 0, Size, bEnumIsSignedInt);

									ensure(SanitizedDefault.Num() == Size);
									Swap(SanitizedDefault, DefaultValue);
								}
								else
								{
									UE_LOGF(LogVariantContent, Error, "Enum property '%ls' points to an enum that doesn't have any valid values!", *GetFullDisplayString());
								}
							}
						}
					}
				}

				// Try to resolve to our parent again, or else we will leave our pointers
				// invalidated or pointing at the CDO
				ClearLastResolve();
				Resolve();
			}
		}
	}

	return DefaultValue;
}

void UPropertyValue::ClearDefaultValue()
{
	DefaultValue.Empty();
}

bool UPropertyValue::IsRecordedDataCurrent()
{
	if (!Resolve())
	{
		return false;
	}

	if (!HasRecordedData())
	{
		return false;
	}

	const TArray<uint8>& RecordedData = GetRecordedData();
	TArray<uint8> CurrentData = GetDataFromResolvedObject();

	FFieldClass* PropClass = GetPropertyClass();
	if (PropClass)
	{
		// All string types are pointer/reference types, and need to be
		// compared carefully
		if (PropClass->IsChildOf(FNameProperty::StaticClass()))
		{
			FName CurrentFName = *((FName*)CurrentData.GetData());
			return TempName.IsEqual(CurrentFName);
		}
		else if (PropClass->IsChildOf(FStrProperty::StaticClass()))
		{
			FString CurrentFString = *((FString*)CurrentData.GetData());
			return TempStr.Equals(CurrentFString);
		}
		else if (PropClass->IsChildOf(FTextProperty::StaticClass()))
		{
			FText CurrentFText = *((FText*)CurrentData.GetData());
			return TempText.EqualTo(CurrentFText);
		}
	}

	// When setting relative rotation, our input rotator value goes through some math
	// that might infinitesimally change its quaternion representation, but nevertheless
	// alter the float representation as a byte array, so we need this explicit check.
	// Note that regular Rotator properties are compared byte-wise, so it should only happen for
	// RelativeRotation
	// See USceneComponent::SetRelativeRotation
	if (PropCategory == EPropertyValueCategory::RelativeRotation)
	{
		const FRotator* RecordedRotator = (const FRotator*)RecordedData.GetData();
		const FRotator* CurrentRotator = (const FRotator*)CurrentData.GetData();

		// Compare via FQuats, because while our recorded rotator will remain correct, it is possible that
		// the rotation on the object is an equivalent rotation instead (like how [0, 95, 0] and [180, 85, 180]
		// are equivallent), and FRotator::Equals would claim they are different
		if ( RecordedRotator && CurrentRotator )
		{
			const FQuat RecordedQuat = RecordedRotator->Quaternion();
			const FQuat CurrentQuat = CurrentRotator->Quaternion();
			return RecordedQuat.Equals( CurrentQuat );
		}
	}
	else if (PropCategory == EPropertyValueCategory::RelativeLocation ||  PropCategory == EPropertyValueCategory::RelativeScale3D)
	{
		const FVector* RecordedVec = (const FVector*)RecordedData.GetData();
		const FVector* CurrentVec = (const FVector*)CurrentData.GetData();

		return RecordedVec->Equals(*CurrentVec);
	}

	return RecordedData == CurrentData;
}

FOnPropertyApplied& UPropertyValue::GetOnPropertyApplied()
{
	return OnPropertyApplied;
}

FOnPropertyRecorded& UPropertyValue::GetOnPropertyRecorded()
{
	return OnPropertyRecorded;
}

FProperty* UPropertyValue::GetProperty() const
{
	return LeafProperty;
}

void UPropertyValue::ApplyViaFunctionSetter(UObject* TargetObject)
{
	//Reference: ScriptCore.cpp, UObject::CallFunctionByNameWithArguments

	if (!TargetObject)
	{
		UE_LOGF(LogVariantContent, Error, "Trying to apply via function setter with a nullptr target object! (UPropertyValue: %ls)", *GetFullDisplayString());
		return;
	}
	if (!PropertySetter)
	{
		UE_LOGF(LogVariantContent, Error, "Trying to apply via function setter with a nullptr function setter! (UPropertyValue: %ls)", *GetFullDisplayString());
		return;
	}

	FProperty* LastParameter = nullptr;

	// find the last parameter
	for ( TFieldIterator<FProperty> It(PropertySetter); It && (It->PropertyFlags&(CPF_Parm|CPF_ReturnParm)) == CPF_Parm; ++It )
	{
		LastParameter = *It;
	}

	// Parse all function parameters.
	uint8* Parms = (uint8*)FMemory_Alloca(PropertySetter->ParmsSize);
	FMemory::Memzero( Parms, PropertySetter->ParmsSize );

	for (TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor))
		{
			LocalProp->InitializeValue_InContainer(Parms);
		}
	}

	const uint32 ExportFlags = PPF_None;
	int32 NumParamsEvaluated = 0;
	bool bAppliedRecordedData = false;

	FFieldClass* ThisValueClass = GetPropertyClass();
	int32 ThisValueSize = GetValueSizeInBytes();
	const TArray<uint8>& RecordedData = GetRecordedData();

	for( TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_OutParm|CPF_ReturnParm); ++It, NumParamsEvaluated++ )
	{
		FProperty* PropertyParam = *It;
		checkSlow(PropertyParam); // Fix static analysis warning

		// Check for a default value
		FString* Defaults = PropertySetterParameterDefaults.Find(PropertyParam->GetName());
		if (Defaults)
		{
			const TCHAR* Result = PropertyParam->ImportText_Direct( **Defaults, PropertyParam->ContainerPtrToValuePtr<uint8>(Parms), NULL, ExportFlags );
			if (!Result)
			{
				UE_LOGF(LogVariantContent, Error, "Failed at applying the default value for parameter '%ls' of PropertyValue '%ls'", *PropertyParam->GetName(), *GetFullDisplayString());
			}
		}

		// Try adding our recorded data bytes
		if (!bAppliedRecordedData && PropertyParam->GetClass() == ThisValueClass)
		{
			bool bParamMatchesThisProperty = true;

			if (ThisValueClass->IsChildOf(FStructProperty::StaticClass()))
			{
				UScriptStruct* ThisStruct = GetStructPropertyStruct();
				UScriptStruct* PropStruct = CastField<FStructProperty>(PropertyParam)->Struct;

				bParamMatchesThisProperty = (ThisStruct == PropStruct);
			}

			if (bParamMatchesThisProperty)
			{
				uint8* StartAddr = It->ContainerPtrToValuePtr<uint8>(Parms);
				FMemory::Memcpy(StartAddr, RecordedData.GetData(), ThisValueSize);
				bAppliedRecordedData = true;
			}
		}
	}

	// HACK: Restore Visibility properties to operating recursively. Temporary until 4.23
	if (PropertySetter->GetName() == TEXT("SetVisibility") && PropertySetter->ParmsSize == 2 && Parms)
	{
		Parms[1] = true;
	}

	// Only actually call the function if we managed to pack our recorded bytes in the params. Else we will
	// just reset the object to defaults
	if (bAppliedRecordedData)
	{
		FEditorScriptExecutionGuard ScriptGuard;
		TargetObject->ProcessEvent(PropertySetter, Parms);
	}
	else
	{
		UE_LOGF(LogVariantContent, Error, "Did not find a parameter that could receive our value of class %ls", *GetPropertyClass()->GetName());
	}

	// Destroy our params
	for( TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm); ++It )
	{
		It->DestroyValue_InContainer(Parms);
	}
}

bool UPropertyValue::ResolveUSCSNodeRecursive(const USCS_Node* Node, int32 SegmentIndex)
{
	// If we're the last segment, we need to capture an actual property, not step into another component,
	// so go back to the regular recursion
	if (SegmentIndex >= (CapturedPropSegments.Num() - 1))
	{
		return ResolvePropertiesRecursive(Node->ComponentClass, Node->ComponentTemplate, SegmentIndex);
	}

	// Step into another USCS_Node
	FCapturedPropSegment& Seg = CapturedPropSegments[SegmentIndex];
	FString TargetComponentNameSuffixed = Seg.ComponentName + TEXT("_GEN_VARIABLE");
	for (const USCS_Node* ChildNode : Node->GetChildNodes())
	{
		if (ChildNode->ComponentClass->IsChildOf(UActorComponent::StaticClass()) &&
			TargetComponentNameSuffixed == ChildNode->ComponentTemplate->GetName())
		{
			return ResolveUSCSNodeRecursive(ChildNode, SegmentIndex + 1);
		}
	}

	return false;
}

bool UPropertyValue::ResolvePropertiesRecursive(UStruct* ContainerClass, void* ContainerAddress, int32 SegmentIndex)
{
	// Adapted from PropertyPathHelpers.cpp because it is incomplete for arrays of UObjects (important for components)

	FCapturedPropSegment& Seg = CapturedPropSegments[SegmentIndex];

	const int32 ArrayIndex = Seg.PropertyIndex == INDEX_NONE ? 0 : Seg.PropertyIndex;

	if (SegmentIndex == 0)
	{
		ParentContainerClass = ContainerClass;
		ParentContainerAddress = ContainerAddress;
	}

	FProperty* Property = FindFProperty<FProperty>(ContainerClass, *Seg.PropertyName);
	if (Property)
	{
		// Not the last link in the chain --> Dig down deeper updating our class/address if we jump an UObjectProp/UStructProp
		if (SegmentIndex < (CapturedPropSegments.Num()-1))
		{
			// Check first to see if this is a simple object (eg. not an array of objects)
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				// If it's an object we need to get the value of the property in the container first before we
				// can continue, if the object is null we safely stop processing the chain of properties.
				if ( UObject* CurrentObject = ObjectProperty->GetPropertyValue_InContainer(ContainerAddress, ArrayIndex) )
				{
					ParentContainerClass = CurrentObject->GetClass();
					ParentContainerAddress = CurrentObject;
					ParentContainerObject = CurrentObject;

					return ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1);
				}
				// We may be using Resolve to resolve on a CDO. If it's a blueprint class, there is no regular
				// component hierarchy to step into, we need to step into its USCS_NODEs
				// Note that we don't need to do this for arrays of UObject properties (e.g. AttachChildren), as we
				// can always reach all the components from expanding from the scene root component through this
				else if (UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(ContainerClass))
				{
					if ((void*)GeneratedClass->GetDefaultObject() == ContainerAddress)
					{
						const TArray<USCS_Node*>& BPNodes = GeneratedClass->SimpleConstructionScript->GetAllNodes();

						// So that we can compare with CDO components, given that they receive UActorComponent::ComponentTemplateNameSuffix
						FString TargetComponentNameSuffixed = Seg.ComponentName + TEXT("_GEN_VARIABLE");

						for (USCS_Node* BPNode : BPNodes)
						{
							if (BPNode->ComponentClass->IsChildOf(ObjectProperty->PropertyClass) &&
								TargetComponentNameSuffixed == BPNode->ComponentTemplate->GetName())
							{
								ParentContainerClass = BPNode->ComponentClass;
								ParentContainerAddress = BPNode->ComponentTemplate;
								ParentContainerObject = BPNode->ComponentTemplate;

								return ResolvePropertiesRecursive(BPNode->ComponentClass, BPNode->ComponentTemplate, SegmentIndex + 1);
							}
						}
					}
				}
			}
			// Check to see if this is a simple weak object property (eg. not an array of weak objects).
			else if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
			{
				FWeakObjectPtr WeakObject = WeakObjectProperty->GetPropertyValue_InContainer(ContainerAddress, ArrayIndex);

				// If it's an object we need to get the value of the property in the container first before we
				// can continue, if the object is null we safely stop processing the chain of properties.
				if ( UObject* CurrentObject = WeakObject.Get() )
				{
					ParentContainerClass = CurrentObject->GetClass();
					ParentContainerAddress = CurrentObject;
					ParentContainerObject = CurrentObject;

					return ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1);
				}
			}
			// Check to see if this is a simple soft object property (eg. not an array of soft objects).
			else if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
			{
				FSoftObjectPtr SoftObject = SoftObjectProperty->GetPropertyValue_InContainer(ContainerAddress, ArrayIndex);

				// If it's an object we need to get the value of the property in the container first before we
				// can continue, if the object is null we safely stop processing the chain of properties.
				if ( UObject* CurrentObject = SoftObject.Get() )
				{
					ParentContainerClass = CurrentObject->GetClass();
					ParentContainerAddress = CurrentObject;
					ParentContainerObject = CurrentObject;

					return ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1);
				}
			}
			// Check to see if this is a simple structure (eg. not an array of structures)
			// Note: We don't actually capture properties *inside* UStructProperties, so this path won't be taken. It is here
			// if we ever wish to change that in the future
			else if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
			{
				void* StructAddress = StructProp->ContainerPtrToValuePtr<void>(ContainerAddress, ArrayIndex);

				ParentContainerClass = StructProp->Struct;
				ParentContainerAddress = StructAddress;

				return ResolvePropertiesRecursive(StructProp->Struct, StructAddress, SegmentIndex + 1);
			}
			else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
			{
				// We have to replicate these cases in here because we need to access the inner properties
				// with the FScriptArrayHelper. If we do another recursive call and try parsing the inner
				// property just as a regular property with an ArrayIndex, it will fail getting the ValuePtr
				// because for some reason properties always have ArrayDim = 1

				FCapturedPropSegment NextSeg = CapturedPropSegments[SegmentIndex+1];

				int32 InnerArrayIndex = NextSeg.PropertyIndex == INDEX_NONE ? 0 : NextSeg.PropertyIndex;

				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(ContainerAddress));

				// In the case of a component, this also ensures we have at least one component in the array, as
				// InnerArrayIndex will always be zero
				if (!ArrayHelper.IsValidIndex(InnerArrayIndex))
				{
					return false;
				}

				// Array properties show up in the path as two entries (one for the array prop and one for the inner)
				// so if we're on the second-to-last path segment, it means we want to capture the inner property, so don't
				// step into it
				// This also handles generic arrays of UObject* and structs without stepping into them (that is,
				// prevents us from going into the ifs below)
				if (SegmentIndex == CapturedPropSegments.Num() - 2)
				{
					LeafProperty = ArrayProp->Inner;
					PropertyValuePtr = ArrayHelper.GetRawPtr(InnerArrayIndex);

					return true;
				}

				if ( FStructProperty* ArrayOfStructsProp = CastField<FStructProperty>(ArrayProp->Inner) )
				{
					void* StructAddress = static_cast<void*>(ArrayHelper.GetRawPtr(InnerArrayIndex));

					ParentContainerClass = ArrayOfStructsProp->Struct;
					ParentContainerAddress = StructAddress;

					// The next link in the chain is just this array's inner. Let's just skip it instead
					return ResolvePropertiesRecursive(ArrayOfStructsProp->Struct, StructAddress, SegmentIndex + 2);
				}
				if ( FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProp->Inner) )
				{
					// If we make it in here we know it's a property inside a component as we don't
					// step into generic UObject properties. We also know its a component of our actor
					// as we don't capture components from other actors

					// This lets us search for the component by name instead, ignoring our InnerArrayIndex
					// This is intuitive because if a component is reordered in the details panel, we kind
					// of expect our bindings to 'follow'.
					if (!NextSeg.ComponentName.IsEmpty())
					{
						for (int32 ComponentIndex = 0; ComponentIndex < ArrayHelper.Num(); ComponentIndex++)
						{
							void* ObjPtrContainer = ArrayHelper.GetRawPtr(ComponentIndex);
							UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ObjPtrContainer);

							if (CurrentObject && CurrentObject->IsA(UActorComponent::StaticClass()) && CurrentObject->GetName() == NextSeg.ComponentName)
							{
								ParentContainerClass = CurrentObject->GetClass();
								ParentContainerAddress =  CurrentObject;
								ParentContainerObject = CurrentObject;

								// The next link in the chain is just this array's inner. Let's just skip it instead
								return ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 2);
							}
						}
					}
					// If we're a property recovered from 4.21, we won't have a component name, so we'll have to
					// try finding our target component by index. We will first check InnerArrayIndex, and if that fails, we
					// will check the other components until we either find something that resolves or we just fall
					// out of this scope
					else
					{
						// First check our actual inner array index
						if (ArrayHelper.IsValidIndex(InnerArrayIndex))
						{
							void* ObjPtrContainer = ArrayHelper.GetRawPtr(InnerArrayIndex);
							UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ObjPtrContainer);
							if (CurrentObject && CurrentObject->IsA(UActorComponent::StaticClass()))
							{
								if (ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 2))
								{
									ParentContainerClass = CurrentObject->GetClass();
									ParentContainerAddress =  CurrentObject;
									ParentContainerObject = CurrentObject;
									NextSeg.ComponentName = CurrentObject->GetName();
									return true;
								}
							}
						}

						// Check every component for something that resolves. It's the best we can do
						for (int32 ComponentIndex = 0; ComponentIndex < ArrayHelper.Num(); ComponentIndex++)
						{
							// Already checked that one
							if (ComponentIndex == InnerArrayIndex)
							{
								continue;
							}

							void* ObjPtrContainer = ArrayHelper.GetRawPtr(ComponentIndex);
							UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ObjPtrContainer);
							if (CurrentObject && CurrentObject->IsA(UActorComponent::StaticClass()))
							{
								if (ResolvePropertiesRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 2))
								{
									ParentContainerClass = CurrentObject->GetClass();
									ParentContainerAddress =  CurrentObject;
									ParentContainerObject = CurrentObject;
									NextSeg.ComponentName = CurrentObject->GetName();
									return true;
								}
							}
						}
					}
				}
			}
			else if( FSetProperty* SetProperty = CastField<FSetProperty>(Property) )
			{
				// TODO: we dont support set properties yet
			}
			else if( FMapProperty* MapProperty = CastField<FMapProperty>(Property) )
			{
				// TODO: we dont support map properties yet
			}
		}
		// Last link, the thing we actually want to capture
		else
		{
			LeafProperty = Property;
			PropertyValuePtr = LeafProperty->ContainerPtrToValuePtr<uint8>(ContainerAddress, ArrayIndex);

			return true;
		}
	}

	ClearLastResolve();
	return false;
}

UPropertyValueTransform::UPropertyValueTransform(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPropertyValueTransform::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Don't need to patch up the property setter name as this won't be used
	if (this->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	switch (PropCategory)
	{
	case EPropertyValueCategory::RelativeLocation:
		PropertySetterName = FName(TEXT("K2_SetRelativeLocation"));
		break;
	case EPropertyValueCategory::RelativeRotation:
		PropertySetterName = FName(TEXT("K2_SetRelativeRotation"));
		break;
	case EPropertyValueCategory::RelativeScale3D:
		PropertySetterName = FName(TEXT("SetRelativeScale3D"));
		break;
	default:
		UE_LOGF(LogVariantContent, Error, "Problem serializing old PropertyValueTransform '%ls'", *GetFullDisplayString());
		break;
	}
}

UPropertyValueVisibility::UPropertyValueVisibility(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPropertyValueVisibility::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Don't need to patch up the property setter name as this won't be used
	if (this->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	switch (PropCategory)
	{
	case EPropertyValueCategory::Visibility:
		PropertySetterName = FName(TEXT("SetVisibility"));
		break;
	default:
		UE_LOGF(LogVariantContent, Error, "Problem serializing old PropertyValueVisibility '%ls'", *GetFullDisplayString());
		break;
	}
}

#undef LOCTEXT_NAMESPACE
