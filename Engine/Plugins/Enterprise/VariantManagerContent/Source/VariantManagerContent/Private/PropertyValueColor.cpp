// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyValueColor.h"

#include "VariantManagerContentLog.h"

#include "Components/LightComponent.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyValueColor)

#define LOCTEXT_NAMESPACE "PropertyValueColor"

UPropertyValueColor::UPropertyValueColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TArray<uint8> UPropertyValueColor::GetDataFromResolvedObject() const
{
	int32 PropertySizeBytes = GetValueSizeInBytes();
	TArray<uint8> CurrentData;
	CurrentData.SetNumZeroed(PropertySizeBytes);

	if (!HasValidResolve())
	{
		return CurrentData;
	}

	// Used by ULightComponent
	if (PropertySetterName == TEXT("SetLightColor"))
	{
		ULightComponent* ContainerObject = (ULightComponent*) ParentContainerAddress;
		if (!ContainerObject || !ContainerObject->IsValidLowLevel())
		{
			UE_LOGF(LogVariantContent, Error, "UPropertyValueColor '%ls' does not have a ULightComponent as parent address!", *GetFullDisplayString());
			return CurrentData;
		}

		FLinearColor Col = ContainerObject->GetLightColor();
		FMemory::Memcpy(CurrentData.GetData(), &Col, PropertySizeBytes);
	}

	return CurrentData;
}

UScriptStruct* UPropertyValueColor::GetStructPropertyStruct() const
{
	static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
	static UScriptStruct* LinearColorScriptStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("LinearColor"));
	return LinearColorScriptStruct;
}

int32 UPropertyValueColor::GetValueSizeInBytes() const
{
	return sizeof(FLinearColor);
}

#undef LOCTEXT_NAMESPACE
