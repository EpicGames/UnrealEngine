// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyValueMaterial.h"

#include "VariantManagerContentLog.h"
#include "VariantObjectBinding.h"

#include "Components/MeshComponent.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyValueMaterial)

#define LOCTEXT_NAMESPACE "PropertyValueMaterial"

FProperty* UPropertyValueMaterial::OverrideMaterialsProperty = nullptr;

UPropertyValueMaterial::UPropertyValueMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMaterialInterface* UPropertyValueMaterial::GetMaterial()
{
	if (!HasRecordedData())
	{
		return nullptr;
	}

	// Go through GetRecordedData because our ValueBytes may be stale, and we need to
	// reload the object through our TempObjPtr
	const TArray<uint8>& RecordedData = GetRecordedData();
	return *(UMaterialInterface**)RecordedData.GetData();
}

void UPropertyValueMaterial::SetMaterial(UMaterialInterface* Mat)
{
	if (Mat && Mat->IsValidLowLevel())
	{
		SetRecordedData((uint8*)&Mat, GetValueSizeInBytes());
	}
}

bool UPropertyValueMaterial::Resolve(UObject* Object)
{
	if (Object == nullptr)
	{
		UVariantObjectBinding* Parent = GetParent();
		if (Parent)
		{
			Object = Parent->GetObject();
		}
	}

	if (Object == nullptr)
	{
		return false;
	}

	if (CapturedPropSegments.Num() == 0)
	{
		return false;
	}

	// Remove an item so that we don't trip an early out in ResolvePropertiesRecursive below
	// (the if(SegmentIndex == CapturedPropSegments.Num() - 2) test). The point of this resolve
	// is just to get ParentContainerAddress pointing at the target UMeshComponent, as we
	// apply/record values by calling the respective functions instead
	FCapturedPropSegment OverrideInner = CapturedPropSegments.Pop();
	ParentContainerObject = Object;
	bool bResolveSucceeded = ResolvePropertiesRecursive(Object->GetClass(), Object, 0);
	CapturedPropSegments.Add(OverrideInner);

	if (!bResolveSucceeded)
	{
		return false;
	}

	if (UMeshComponent* ContainerObject = (UMeshComponent*) ParentContainerAddress)
	{
		int32 NumSegs = CapturedPropSegments.Num();
		if (NumSegs < 1)
		{
			return false;
		}

		// Can't resolve as we don't have as many material slots as the property path requires
		int32 MatIndex = CapturedPropSegments[NumSegs-1].PropertyIndex;
		if (!ContainerObject->GetMaterialSlotNames().IsValidIndex(MatIndex))
		{
			return false;
		}
	}
	else
	{
		return false;
	}

	// We don't want anything trying to access this property by itself
	PropertyValuePtr = nullptr;
	LeafProperty = nullptr;
	PropertySetter = nullptr;
	return true;
}

bool UPropertyValueMaterial::ContainsProperty(const FProperty* Prop) const
{
	if (OverrideMaterialsProperty == nullptr)
	{
		if (FArrayProperty* ArrayProp = FindFProperty<FArrayProperty>(UMeshComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials)))
		{
			OverrideMaterialsProperty = ArrayProp->Inner;
		}
	}

	return OverrideMaterialsProperty ? Prop == OverrideMaterialsProperty : false;
}

UStruct* UPropertyValueMaterial::GetPropertyParentContainerClass() const
{
	return UMeshComponent::StaticClass();
}

TArray<uint8> UPropertyValueMaterial::GetDataFromResolvedObject() const
{
	TArray<uint8> CurrentData;
	int32 PropertySizeBytes = GetValueSizeInBytes();
	CurrentData.SetNumZeroed(PropertySizeBytes);

	if (!HasValidResolve())
	{
		return CurrentData;
	}

	UMeshComponent* ContainerObject = (UMeshComponent*) ParentContainerAddress;
	if (!ContainerObject)
	{
		UE_LOGF(LogVariantContent, Error, "UPropertyValueMaterial '%ls' does not have a UMeshComponent as parent address!", *GetFullDisplayString());
		return CurrentData;
	}

	int32 NumSegs = CapturedPropSegments.Num();
	if (NumSegs < 1)
	{
		return CurrentData;
	}

	int32 MatIndex = CapturedPropSegments[NumSegs-1].PropertyIndex;
	UMaterialInterface* Mat = ContainerObject->GetMaterial(MatIndex);

	FMemory::Memcpy(CurrentData.GetData(), (uint8*)&Mat, PropertySizeBytes);
	return CurrentData;
}

void UPropertyValueMaterial::ApplyDataToResolvedObject()
{
	if (!HasRecordedData())
	{
		return;
	}

	if (!Resolve())
	{
		return;
	}

	// Modify owner actor
	UObject* ContainerOwnerObject = nullptr;
	if (UVariantObjectBinding* Parent = GetParent())
	{
		UObject* OwnerActor = Parent->GetObject();
		if (OwnerActor)
		{
			OwnerActor->SetFlags(RF_Transactional);
			OwnerActor->Modify();
			ContainerOwnerObject = OwnerActor;
		}
	}

	// Modify container component
	UMeshComponent* ContainerObject = nullptr;
	if (UMeshComponent* MeshComponent = (UMeshComponent*) ParentContainerAddress)
	{
		if(MeshComponent)
		{
			MeshComponent->SetFlags(RF_Transactional);
			MeshComponent->Modify();
			ContainerObject = MeshComponent;
		}
	}

	if (!ContainerObject)
	{
		UE_LOGF(LogVariantContent, Error, "UPropertyValueMaterial '%ls' does not have a UMeshComponent as parent address!", *GetFullDisplayString());
		return;
	}

	// Go through GetRecordedData to resolve our path if we need to
	UMaterialInterface* Mat = *((UMaterialInterface**)GetRecordedData().GetData());
	if ( !Mat )
	{
		UE_LOGF( LogVariantContent, Error, "Failed to apply recorded data from UPropertyValueMaterial '%ls'!", *GetFullDisplayString() );
		return;
	}

	int32 NumSegs = CapturedPropSegments.Num();
	if (NumSegs > 0)
	{
		int32 MatIndex = CapturedPropSegments[NumSegs-1].PropertyIndex;
		ContainerObject->SetMaterial(MatIndex, Mat);
	}

	OnPropertyApplied.Broadcast();
}

FFieldClass* UPropertyValueMaterial::GetPropertyClass() const
{
	return FObjectProperty::StaticClass();
}

UClass* UPropertyValueMaterial::GetObjectPropertyObjectClass() const
{
	return UMaterialInterface::StaticClass();
}

int32 UPropertyValueMaterial::GetValueSizeInBytes() const
{
	return sizeof(UMaterialInterface*);
}

const TArray<uint8>& UPropertyValueMaterial::GetDefaultValue()
{
	if (DefaultValue.Num() == 0)
	{
		if (UVariantObjectBinding* Binding = GetParent())
		{
			if (UObject* Object = Binding->GetObject())
			{
				int32 NumBytes = GetValueSizeInBytes();
				DefaultValue.SetNumZeroed(NumBytes);

				if (Resolve(Object->GetClass()->GetDefaultObject()))
				{
					if (UMeshComponent* ContainerObject = (UMeshComponent*) ParentContainerAddress)
					{
						int32 NumSegs = CapturedPropSegments.Num();
						if (NumSegs > 0)
						{
							int32 MatIndex = CapturedPropSegments[NumSegs-1].PropertyIndex;

							if (!ContainerObject->GetMaterialSlotNames().IsValidIndex(MatIndex))
							{
								UE_LOGF(LogVariantContent, Warning, "Tried to fetch non-existent material with index %d for object '%ls'! Setting default as nullptr instead", MatIndex, *Object->GetName());
							}

							// This might be nullptr if index is invalid but that's exactly what we want anyway
							UMaterialInterface* DefaultMat = ContainerObject->GetMaterial(MatIndex);
							FMemory::Memcpy(DefaultValue.GetData(), &DefaultMat, NumBytes);
						}
					}
				}

				// Try to resolve to our parent again, or else we will leave our pointers
				// invalidated or pointing at the CDO
				ClearLastResolve();
				Resolve();
			}
		}
	}

	return DefaultValue;
}

#undef LOCTEXT_NAMESPACE
