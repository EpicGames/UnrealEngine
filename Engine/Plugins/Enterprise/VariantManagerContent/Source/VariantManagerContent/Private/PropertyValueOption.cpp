// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyValueOption.h"

#include "SwitchActor.h"
#include "VariantObjectBinding.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyValueOption)

#define LOCTEXT_NAMESPACE "PropertyValueOption"

UPropertyValueOption::UPropertyValueOption(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UPropertyValueOption::Resolve(UObject* Object /*= nullptr*/)
{
	if (Object == nullptr)
	{
		UVariantObjectBinding* Parent = GetParent();
		if (Parent)
		{
			Object = Parent->GetObject();
		}
	}

	if (ASwitchActor* SwitchActor = Cast<ASwitchActor>(Object))
	{
		ParentContainerClass = ASwitchActor::StaticClass();
		ParentContainerAddress = SwitchActor;
		return true;
	}

	ClearLastResolve();
	return false;
}

TArray<uint8> UPropertyValueOption::GetDataFromResolvedObject() const
{
	int32 PropertySizeBytes = GetValueSizeInBytes();
	TArray<uint8> CurrentData;
	CurrentData.SetNumZeroed(PropertySizeBytes);

	if (!HasValidResolve())
	{
		return CurrentData;
	}

	// We know it's valid from HasValidResolve()
	ASwitchActor* ResolvedObject = (ASwitchActor*)ParentContainerAddress;

	int32 SelectedIndex = ResolvedObject->GetSelectedOption();
	FMemory::Memcpy(CurrentData.GetData(), &SelectedIndex, GetValueSizeInBytes());

	return CurrentData;
}

void UPropertyValueOption::ApplyDataToResolvedObject()
{
	if (!HasRecordedData() || !Resolve())
	{
		return;
	}

	// We know it's valid from HasValidResolve()
	ASwitchActor* ResolvedObject = (ASwitchActor*)ParentContainerAddress;

	const TArray<uint8>& RecordedData = GetRecordedData();
	int32 RecordedOption = *((int32*)RecordedData.GetData());
	ResolvedObject->SelectOption(RecordedOption);
}

const TArray<uint8>& UPropertyValueOption::GetDefaultValue()
{
	if (DefaultValue.Num() == 0)
	{
		int32 IndexNone = INDEX_NONE;
		DefaultValue.SetNumUninitialized(sizeof(int32));
		FMemory::Memcpy(DefaultValue.GetData(), &IndexNone, sizeof(IndexNone));
	}

	return DefaultValue;
}

int32 UPropertyValueOption::GetValueSizeInBytes() const
{
	return sizeof(int32);
}

#undef LOCTEXT_NAMESPACE
