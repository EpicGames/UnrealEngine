// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyValueSoftObject.h"

#include "LevelVariantSets.h"
#include "VariantManagerContentLog.h"
#include "VariantObjectBinding.h"

#include "Components/ActorComponent.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyValueSoftObject)

#define LOCTEXT_NAMESPACE "PropertyValueSoftObject"

UPropertyValueSoftObject::UPropertyValueSoftObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UPropertyValueSoftObject::GetValueSizeInBytes() const
{
	return sizeof(UObject*);
}

FFieldClass* UPropertyValueSoftObject::GetPropertyClass() const
{
	return FSoftObjectProperty::StaticClass();
}

void UPropertyValueSoftObject::ApplyDataToResolvedObject()
{
	if (!HasRecordedData() || !Resolve())
	{
		return;
	}

	// Modify owner actor
	UObject* ContainerOwnerObject = nullptr;
	if (UVariantObjectBinding* Parent = GetParent())
	{
		if (UObject* OwnerActor = Parent->GetObject())
		{
			OwnerActor->SetFlags(RF_Transactional);
			OwnerActor->Modify();
			ContainerOwnerObject = OwnerActor;
		}
	}

	// Modify container component
	UObject* ContainerObject = nullptr;
	if (ParentContainerObject && ParentContainerObject->IsA(UActorComponent::StaticClass()))
	{
		ParentContainerObject->SetFlags(RF_Transactional);
		ParentContainerObject->Modify();
		ContainerObject = ParentContainerObject;
	}

	if (PropertySetter)
	{
		// If we resolved, these are valid
		ApplyViaFunctionSetter((UObject*)ParentContainerAddress);
	}
	else if(FSoftObjectProperty* SoftProperty = CastField<FSoftObjectProperty>(GetProperty()))
	{
		// Never access ValueBytes directly as we might need to fixup UObjectProperty values
		const TArray<uint8>& RecordedData = GetRecordedData();
		UObject* RecordedPtr = *((UObject**)RecordedData.GetData());
		SoftProperty->SetObjectPropertyValue(PropertyValuePtr, RecordedPtr);
	}

#if WITH_EDITOR
	if (ContainerObject)
	{
		ContainerObject->PostEditChange();
	}
	if (ContainerOwnerObject && ContainerOwnerObject != ContainerObject)
	{
		ContainerOwnerObject->PostEditChange();
	}
#endif
	OnPropertyApplied.Broadcast();
}

TArray<uint8> UPropertyValueSoftObject::GetDataFromResolvedObject() const
{
	int32 PropertySizeBytes = GetValueSizeInBytes();
	TArray<uint8> CurrentData;
	CurrentData.SetNumZeroed(PropertySizeBytes);

	if (!HasValidResolve())
	{
		return CurrentData;
	}

	if (FSoftObjectProperty* SoftProperty = CastField<FSoftObjectProperty>(GetProperty()))
	{
		UObject* CurrentObj = SoftProperty->GetObjectPropertyValue(PropertyValuePtr);
		FMemory::Memcpy(CurrentData.GetData(), (uint8*)&CurrentObj, PropertySizeBytes);
	}

	return CurrentData;
}

void UPropertyValueSoftObject::ApplyViaFunctionSetter(UObject* TargetObject)
{
	//Reference: ScriptCore.cpp, UObject::CallFunctionByNameWithArguments

	if (!TargetObject)
	{
		UE_LOGF(LogVariantContent, Error, "Trying to apply via function setter with a nullptr target object! (UPropertyValue: %ls)", *GetFullDisplayString());
		return;
	}
	if (!PropertySetter)
	{
		UE_LOGF(LogVariantContent, Error, "Trying to apply via function setter with a nullptr function setter! (UPropertyValue: %ls)", *GetFullDisplayString());
		return;
	}

	FProperty* LastParameter = nullptr;

	// find the last parameter
	for ( TFieldIterator<FProperty> It(PropertySetter); It && (It->PropertyFlags&(CPF_Parm|CPF_ReturnParm)) == CPF_Parm; ++It )
	{
		LastParameter = *It;
	}

	// Parse all function parameters.
	uint8* Parms = (uint8*)FMemory_Alloca(PropertySetter->ParmsSize);
	FMemory::Memzero( Parms, PropertySetter->ParmsSize );

	for (TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor))
		{
			LocalProp->InitializeValue_InContainer(Parms);
		}
	}

	const uint32 ExportFlags = PPF_None;
	int32 NumParamsEvaluated = 0;
	bool bAppliedRecordedData = false;

	FFieldClass* ThisValueClass = GetPropertyClass();
	int32 ThisValueSize = GetValueSizeInBytes();
	const TArray<uint8>& RecordedData = GetRecordedData();
	UObject* RecordedPtr = *((UObject**)RecordedData.GetData());

	for( TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_OutParm|CPF_ReturnParm); ++It, NumParamsEvaluated++ )
	{
		FProperty* PropertyParam = *It;
		checkSlow(PropertyParam); // Fix static analysis warning

		// Check for a default value
		FString* Defaults = PropertySetterParameterDefaults.Find(PropertyParam->GetName());
		if (Defaults)
		{
			const TCHAR* Result = PropertyParam->ImportText_Direct( **Defaults, PropertyParam->ContainerPtrToValuePtr<uint8>(Parms), NULL, ExportFlags );
			if (!Result)
			{
				UE_LOGF(LogVariantContent, Error, "Failed at applying the default value for parameter '%ls' of PropertyValue '%ls'", *PropertyParam->GetName(), *GetFullDisplayString());
			}
		}

		// Try adding our recorded data bytes
		if (!bAppliedRecordedData && PropertyParam->GetClass() == ThisValueClass)
		{
			bool bParamMatchesThisProperty = true;

			if (ThisValueClass->IsChildOf(FStructProperty::StaticClass()))
			{
				UScriptStruct* ThisStruct = GetStructPropertyStruct();
				UScriptStruct* PropStruct = CastField<FStructProperty>(PropertyParam)->Struct;

				bParamMatchesThisProperty = (ThisStruct == PropStruct);
			}

			if (bParamMatchesThisProperty)
			{
				uint8* StartAddr = It->ContainerPtrToValuePtr<uint8>(Parms);

				if (FSoftObjectProperty* SoftPropertyParam = CastField<FSoftObjectProperty>(PropertyParam))
				{
					SoftPropertyParam->SetObjectPropertyValue(StartAddr, RecordedPtr);
				}
				else
				{
					FMemory::Memcpy(StartAddr, RecordedData.GetData(), ThisValueSize);
				}

				bAppliedRecordedData = true;
			}
		}
	}

	// HACK: Restore Visibility properties to operating recursively. Temporary until 4.23
	if (PropertySetter->GetName() == TEXT("SetVisibility") && PropertySetter->ParmsSize == 2 && Parms)
	{
		Parms[1] = true;
	}

	// Only actually call the function if we managed to pack our recorded bytes in the params. Else we will
	// just reset the object to defaults
	if (bAppliedRecordedData)
	{
		FEditorScriptExecutionGuard ScriptGuard;
		TargetObject->ProcessEvent(PropertySetter, Parms);
	}
	else
	{
		UE_LOGF(LogVariantContent, Error, "Did not find a parameter that could receive our value of class %ls", *GetPropertyClass()->GetName());
	}

	// Destroy our params
	for( TFieldIterator<FProperty> It(PropertySetter); It && It->HasAnyPropertyFlags(CPF_Parm); ++It )
	{
		It->DestroyValue_InContainer(Parms);
	}
}

namespace PropertyValueSoftImpl
{
	/**
	* Turns a path like '/Game/UEDPIE_0_Untitled.Untitled:PersistentLevel.Sphere_5'
	* into '/Game/Untitled.Untitled:PersistentLevel.Sphere_5'. Does not check if the resulting
	* path resolves.
	*/
	FSoftObjectPath UnFixupForPIE(FSoftObjectPath& PIEPath, int32 PIEInstanceID)
	{
		FString Path = PIEPath.ToString();
		if (FPackageName::GetLongPackageAssetName(Path).StartsWith(PLAYWORLD_PACKAGE_PREFIX))
		{
			FString PIEPrefix = FString::Printf(TEXT("%s_%d_"), PLAYWORLD_PACKAGE_PREFIX, PIEInstanceID);
			Path = Path.Replace(*PIEPrefix, TEXT(""), ESearchCase::CaseSensitive);
		}

		return FSoftObjectPath{Path};
	}
}

bool UPropertyValueSoftObject::IsRecordedDataCurrent()
{
	if (!Resolve())
	{
		return false;
	}

	if (!HasRecordedData())
	{
		return false;
	}

	const TArray<uint8>& RecordedData = GetRecordedData();
	TArray<uint8> CurrentData = GetDataFromResolvedObject();

#if WITH_EDITOR
	// It could be that CurrentData may be a reference to a PIE object, in which
	// case we need to UnFix it in order to compare

	if (ULevelVariantSets* LVS = GetTypedOuter<ULevelVariantSets>())
	{
		int32 PIEInstanceID;
		UWorld* World = LVS->GetWorldContext(PIEInstanceID);

		if (PIEInstanceID != INDEX_NONE)
		{
			UObject* RecordedObj = *((UObject**)RecordedData.GetData());
			UObject* CurrentObj = *((UObject**)CurrentData.GetData());

			FSoftObjectPath CurrentPath = CurrentObj;
			FSoftObjectPath RecordedPath = RecordedObj;

			// RecordedPath is always unfixed during SetRecordedData
			CurrentPath = PropertyValueSoftImpl::UnFixupForPIE(CurrentPath, PIEInstanceID);
			return CurrentPath == RecordedPath;
		}
	}
#endif

	return RecordedData == CurrentData;
}

void UPropertyValueSoftObject::SetRecordedData(const uint8* NewDataBytes, int32 NumBytes, int32 Offset /*= 0*/)
{
	if (NumBytes != sizeof(UObject*))
	{
		UE_LOG(LogVariantContent, Log, TEXT("Attempting to set a UPropertyValueSoftObject with data that is %d bytes long, instead of %" SIZE_T_FMT "! It should receive just an object pointer, and not a full FSoftObjectPtr!"), NumBytes, sizeof(UObject*));
		return;
	}

#if WITH_EDITOR
	if (ULevelVariantSets* LVS = GetTypedOuter<ULevelVariantSets>())
	{
		int32 PIEInstanceID;
		UWorld* World = LVS->GetWorldContext(PIEInstanceID);

		if (PIEInstanceID != INDEX_NONE)
		{
			// Try removing the PIE prefix and capture the underlying Editor actor or we won't be able to resolve
			// this Actor anymore once we come out of PIE.
			UObject* NewObj = *((UObject**)NewDataBytes);
			FSoftObjectPath Path = NewObj;
			Path = PropertyValueSoftImpl::UnFixupForPIE(Path, PIEInstanceID);

			// It's OK if this is nullptr: It means we can't find this actor in the Editor world and so we really
			// should clear out our recorded data, or else it will point at random data once PIE ends
			UObject* EditorObj = Path.TryLoad();
			if (NewObj != nullptr && EditorObj == nullptr)
			{
				UE_LOGF(LogVariantContent, Error, "Failed to record Actor '%ls' into property '%ls': The actor could not be found in the Editor world. Note that it is not possible to record references to temporary actors!", *NewObj->GetName(), *GetFullDisplayString());
			}

			Super::SetRecordedData((uint8*)&EditorObj, NumBytes, Offset);
			return;
		}
	}
#endif

	Super::SetRecordedData(NewDataBytes, NumBytes, Offset);
}

#undef LOCTEXT_NAMESPACE
