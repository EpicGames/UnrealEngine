// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchActor.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SwitchActor)

namespace SwitchActorImpl
{
	int32 DiscoverSelectedOption(const TArray<AActor*>& Actors)
	{
		int32 SingleVisibleChildIndex = INDEX_NONE;
		for (int32 Index = 0; Index < Actors.Num(); ++Index)
		{
			const AActor* Actor = Actors[Index];
			if (USceneComponent* ActorRoot = Actor->GetRootComponent())
			{
				if (ActorRoot->IsVisible())
				{
					if (SingleVisibleChildIndex == INDEX_NONE)
					{
						SingleVisibleChildIndex = Index;
					}
					else
					{
						SingleVisibleChildIndex = INDEX_NONE;
						break;
					}
				}
			}
		}

		return SingleVisibleChildIndex;
	}
}

ASwitchActor::ASwitchActor(const FObjectInitializer& Init)
	: Super(Init)
{
	SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
	SceneComponent->SetMobility(EComponentMobility::Static);

	RootComponent = SceneComponent;
}

TArray<AActor*> ASwitchActor::GetOptions() const
{
	const bool bResetArray = false;
	TArray<AActor*> Result;
	GetAttachedActors(Result, bResetArray);

	// We have to sort these by FName because the attach order is not guaranteed
	// It seems to invert when going into PIE, for example
	Algo::Sort(Result, [](const AActor* LHS, const AActor* RHS)
	{
		return LHS->GetName() < RHS->GetName();
	});

	return Result;
}

int32 ASwitchActor::GetSelectedOption() const
{
	return LastSelectedOption;
}

void ASwitchActor::SelectOption(int32 OptionIndex)
{
	TArray<AActor*> Actors = GetOptions();

	if (!Actors.IsValidIndex(OptionIndex))
	{
		return;
	}

	Modify();

	// So that we set our children to visible only if we ourselves are visible,
	// which helps this actor behave nicely when child of another switchactor that hasn't selected us
	bool bOurVisibility = this->GetRootComponent()->GetVisibleFlag();

	for (int32 Index = 0; Index < Actors.Num(); ++Index)
	{
		AActor* Actor = Actors[Index];
		if (ASwitchActor* ChildSwitchActor = Cast<ASwitchActor>(Actor))
		{
			ChildSwitchActor->SetVisibility(bOurVisibility && Index == OptionIndex);
		}
		else if(Actor)
		{
			if (USceneComponent* ActorRootComponent = Actor->GetRootComponent())
			{
				ActorRootComponent->Modify();

				const bool bPropagateToChildren = true;
				ActorRootComponent->SetVisibility(bOurVisibility && Index == OptionIndex, bPropagateToChildren);
			}
		}
	}

	LastSelectedOption = OptionIndex;
	OnSwitchActorSwitch.Broadcast(OptionIndex);
}

FOnSwitchActorSwitch& ASwitchActor::GetOnSwitchDelegate()
{
	return OnSwitchActorSwitch;
}

void ASwitchActor::SetVisibility(bool bVisible)
{
	Modify();

	const bool bPropagateToChildren = true;
	this->GetRootComponent()->SetVisibility(bVisible, bPropagateToChildren);
	SelectOption(LastSelectedOption);
}

void ASwitchActor::PostLoad()
{
	Super::PostLoad();

	if (LastSelectedOption == INDEX_NONE)
	{
		LastSelectedOption = SwitchActorImpl::DiscoverSelectedOption(GetOptions());
	}
}

