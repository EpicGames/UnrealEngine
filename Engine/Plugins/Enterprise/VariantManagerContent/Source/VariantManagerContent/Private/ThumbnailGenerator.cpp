// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailGenerator.h"

#include "VariantManagerContentLog.h"

#include "CanvasTypes.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "HAL/PlatformFileManager.h"
#include "ImageUtils.h"
#include "LegacyScreenPercentageDriver.h"
#include "Math/InverseRotationMatrix.h"
#include "SceneInterface.h"
#include "RenderingThread.h"
#include "RendererInterface.h"
#include "TextureResource.h"

#if WITH_EDITOR
#include "Editor.h"
#include "RendererInterface.h"
#include "TextureCompiler.h"
#include "ObjectTools.h"
#else
#include "Engine/World.h"
#endif

namespace ThumbnailGeneratorImpl
{
    void RenderSceneToTexture(
		FSceneInterface* Scene,
		const FVector& ViewOrigin,
		const FMatrix& ViewRotationMatrix,
		const FMatrix& ProjectionMatrix,
		FIntPoint TargetSize,
		float TargetGamma,
		TArray<FColor>& OutSamples)
	{
		UTextureRenderTarget2D* RenderTargetTexture = NewObject<UTextureRenderTarget2D>();
		RenderTargetTexture->AddToRoot();
		RenderTargetTexture->ClearColor = FLinearColor::Transparent;
		RenderTargetTexture->TargetGamma = TargetGamma;
		RenderTargetTexture->InitCustomFormat(TargetSize.X, TargetSize.Y, PF_B8G8R8A8, false);

		FTextureRenderTargetResource* RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();

		FSceneViewFamilyContext ViewFamily(
			FSceneViewFamily::ConstructionValues(RenderTargetResource, Scene, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime::GetTimeSinceAppStart())
		);

		ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
			ViewFamily, /* GlobalResolutionFraction = */ 1.0f));

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, TargetSize.X, TargetSize.Y));
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.ViewOrigin = ViewOrigin;
		ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
		ViewInitOptions.ProjectionMatrix = ProjectionMatrix;

		FSceneView* NewView = new FSceneView(ViewInitOptions);
		ViewFamily.Views.Add(NewView);

		FCanvas Canvas(RenderTargetResource, NULL, FGameTime::GetTimeSinceAppStart(), Scene->GetFeatureLevel());
		Canvas.Clear(FLinearColor::Transparent);
		GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);

		// Copy the contents of the remote texture to system memory
		OutSamples.SetNumUninitialized(TargetSize.X * TargetSize.Y);
		RenderTargetResource->ReadPixelsPtr(OutSamples.GetData(), FReadSurfaceDataFlags(), FIntRect(0, 0, TargetSize.X, TargetSize.Y));
		FlushRenderingCommands();

		RenderTargetTexture->RemoveFromRoot();
		RenderTargetTexture = nullptr;
	}

	bool IsPixelFormatResizeable( EPixelFormat PixelFormat )
	{
		return
			PixelFormat == PF_A8R8G8B8 ||
			PixelFormat == PF_R8G8B8A8 ||
			PixelFormat == PF_B8G8R8A8 ||
			PixelFormat == PF_R8G8B8A8_SNORM ||
			PixelFormat == PF_R8G8B8A8_UINT;
	}

	// This function works like ImageUtils::CreateTexture, except that it should also work at runtime
	// Note that it's entirely possible that Bytes is a compressed (e.g. DXT1) byte buffer, although we won't be able to copy source data in that case
    UTexture2D* CreateTextureFromBulkData(uint32 Width, uint32 Height, const void* Bytes, uint64 NumBytes, EPixelFormat PixelFormat, bool bSetSourceData = false)
    {
        UTexture2D* Texture = UTexture2D::CreateTransient(Width, Height, PixelFormat);
        if (Texture)
        {
#if WITH_EDITOR
			if ( bSetSourceData && IsPixelFormatResizeable(PixelFormat) )
			{
				// Set via Source or else it won't be saved to disk. This is adapted from FImageUtils::CreateTexture2D,
				// so we don't have to match it's signature with a TArray<FColor>
				Texture->Source.Init( Width, Height, /*NumSlices=*/ 1, /*NumMips=*/ 1, TSF_BGRA8 );

				uint8* MipData = Texture->Source.LockMip( 0 );
				for ( uint32 Y = 0; Y < Height; Y++ )
				{
					uint64 Index = ( Height - 1 - Y ) * Width * sizeof( FColor );
					uint8* DestPtr = &MipData[ Index ];

					FMemory::Memcpy( DestPtr, reinterpret_cast< const uint8* >( Bytes ) + Index, Width * sizeof( FColor ) );
				}
				Texture->Source.UnlockMip( 0 );

				Texture->SRGB = true;
				Texture->CompressionSettings = TC_EditorIcon;
				Texture->MipGenSettings = TMGS_FromTextureGroup;
				Texture->DeferCompression = true;
				Texture->PostEditChange();
			}
			else
#endif // WITH_EDITOR
			{
				void* Data = Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
				{
					FMemory::Memcpy(Data, Bytes, NumBytes);
				}
				Texture->GetPlatformData()->Mips[0].BulkData.Unlock();

				Texture->GetPlatformData()->SetNumSlices(1);
				Texture->UpdateResource();
			}
		}

        return Texture;
    }
}

UTexture2D* ThumbnailGenerator::GenerateThumbnailFromTexture(UTexture2D* Texture)
{
	if ( !Texture )
	{
		return nullptr;
	}

    // Force all mips to stream in, as we may need to use mip 0 for the thumbnail
#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation( { Texture } );
#endif // WITH_EDITOR
	Texture->SetForceMipLevelsToBeResident(30.0f);
	Texture->WaitForStreaming();

	int32 TargetWidth = Texture->GetSizeX();
	int32 TargetHeight = Texture->GetSizeY();

	if (TargetWidth == 0 || TargetHeight == 0 || !Texture->GetResource())
	{
		UE_LOGF(LogVariantContent, Error, "Failed create a thumbnail from texture '%ls'", *Texture->GetName());
		return nullptr;
	}

	if (TargetWidth > VARIANT_MANAGER_THUMBNAIL_SIZE || TargetHeight > VARIANT_MANAGER_THUMBNAIL_SIZE)
	{
		if (TargetWidth >= TargetHeight)
		{
			TargetHeight = (int)(VARIANT_MANAGER_THUMBNAIL_SIZE * TargetHeight / (float)TargetWidth);
			TargetWidth = VARIANT_MANAGER_THUMBNAIL_SIZE;
		}
		else
		{
			TargetWidth = (int)(VARIANT_MANAGER_THUMBNAIL_SIZE * TargetWidth / (float)TargetHeight);
			TargetHeight = VARIANT_MANAGER_THUMBNAIL_SIZE;
		}
	}

	UTextureRenderTarget2D* RenderTargetTexture = NewObject<UTextureRenderTarget2D>();
	RenderTargetTexture->AddToRoot();
	RenderTargetTexture->ClearColor = FLinearColor::Transparent;
	RenderTargetTexture->TargetGamma = 0.f;
	RenderTargetTexture->InitCustomFormat(TargetWidth, TargetHeight, PF_B8G8R8A8, false);

	FTextureRenderTargetResource* RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();

	const double Time = FApp::GetCurrentTime() - GStartTime;
	FCanvas Canvas(RenderTargetResource, NULL, FGameTime::GetTimeSinceAppStart(), GWorld->Scene->GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	const bool bAlphaBlend = false;

	Canvas.DrawTile(
		0.0f,
		0.0f,
		(float)TargetWidth,
		(float)TargetHeight,
		0.0f,
		0.0f,
		1.0f,
		1.0f,
		FLinearColor::White,
		Texture->GetResource(),
		bAlphaBlend);
	Canvas.Flush_GameThread();

	// Copy the contents of the remote texture to system memory
	TArray<FColor> CapturedImage;
	CapturedImage.SetNumUninitialized(TargetWidth * TargetHeight);
	RenderTargetResource->ReadPixelsPtr(CapturedImage.GetData(), FReadSurfaceDataFlags(), FIntRect(0, 0, TargetWidth, TargetHeight));

	RenderTargetTexture->RemoveFromRoot();
	RenderTargetTexture = nullptr;

	const bool bSetSourceData = true;
	UTexture2D* Thumbnail = ThumbnailGeneratorImpl::CreateTextureFromBulkData(
		TargetWidth,
		TargetHeight,
		(void*)CapturedImage.GetData(),
		CapturedImage.Num() * sizeof(FColor),
		PF_B8G8R8A8,
		bSetSourceData
	);

    if (Thumbnail == nullptr)
	{
		UE_LOGF(LogVariantContent, Warning, "Failed to generate thumbnail from texture '%ls'", *Texture->GetName());
	}

    return Thumbnail;
}

UTexture2D* ThumbnailGenerator::GenerateThumbnailFromFile(FString FilePath)
{
    if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FilePath))
	{
		if (UTexture2D* OriginalTexture = FImageUtils::ImportFileAsTexture2D(FilePath))
		{
			return GenerateThumbnailFromTexture(OriginalTexture);
		}
	}

	return nullptr;
}

UTexture2D* ThumbnailGenerator::GenerateThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees, float MinZ, float Gamma)
{
    if (!WorldContextObject)
	{
		return nullptr;
	}

	UWorld* World = WorldContextObject->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	FSceneInterface* Scene = World->Scene;

	TArray<FColor> CapturedImage;
	CapturedImage.SetNumUninitialized(VARIANT_MANAGER_THUMBNAIL_SIZE * VARIANT_MANAGER_THUMBNAIL_SIZE);

	FIntPoint Size{VARIANT_MANAGER_THUMBNAIL_SIZE, VARIANT_MANAGER_THUMBNAIL_SIZE};

	ThumbnailGeneratorImpl::RenderSceneToTexture(
		Scene,
		CameraTransform.GetTranslation(),
		FInverseRotationMatrix(CameraTransform.Rotator()) * FInverseRotationMatrix(FRotator(0, -90, 90)),
		FReversedZPerspectiveMatrix(FOVDegrees * 2, 1, 1, MinZ),
		Size,
		Gamma,
		CapturedImage);

	const bool bSetSourceData = true;
	UTexture2D* Thumbnail = ThumbnailGeneratorImpl::CreateTextureFromBulkData(
		VARIANT_MANAGER_THUMBNAIL_SIZE,
		VARIANT_MANAGER_THUMBNAIL_SIZE,
		( void* ) CapturedImage.GetData(),
		CapturedImage.Num() * sizeof( FColor ),
		PF_B8G8R8A8,
		bSetSourceData
	);

    if (Thumbnail == nullptr)
	{
		UE_LOGF(LogVariantContent, Warning, "Failed to resize texture thumbnail texture from camera!");
	}

    return Thumbnail;
}

UTexture2D* ThumbnailGenerator::GenerateThumbnailFromEditorViewport()
{
#if WITH_EDITOR
	// Check for this too because WITH_EDITOR is still defined for Standalone mode
	if ( !GIsEditor || !GEditor )
	{
		return nullptr;
	}

	FViewport* Viewport = GEditor->GetActiveViewport();

	if (!GCurrentLevelEditingViewportClient || !Viewport)
	{
		return nullptr;
	}

	FLevelEditorViewportClient* OldViewportClient = GCurrentLevelEditingViewportClient;
	// Remove selection box around client during render
	GCurrentLevelEditingViewportClient = nullptr;
	Viewport->Draw();

	uint32 SrcWidth = Viewport->GetSizeXY().X;
	uint32 SrcHeight = Viewport->GetSizeXY().Y;
	TArray<FColor> OrigBitmap;
	if (!Viewport->ReadPixels(OrigBitmap) || OrigBitmap.Num() != SrcWidth * SrcHeight)
	{
		return nullptr;
	}

	// Pre-resize the image because we already it in FColor array form anyway, which should make SetThumbnailFromTexture skip most of its processing
	TArray<FColor> ScaledBitmap;
	FImageUtils::CropAndScaleImage(SrcWidth, SrcHeight, VARIANT_MANAGER_THUMBNAIL_SIZE, VARIANT_MANAGER_THUMBNAIL_SIZE, OrigBitmap, ScaledBitmap);

	// Redraw viewport to have the yellow highlight again
	GCurrentLevelEditingViewportClient = OldViewportClient;
	Viewport->Draw();

	const bool bSetSourceData = true;
	UTexture2D* Thumbnail = ThumbnailGeneratorImpl::CreateTextureFromBulkData(
		VARIANT_MANAGER_THUMBNAIL_SIZE,
		VARIANT_MANAGER_THUMBNAIL_SIZE,
		(void*)ScaledBitmap.GetData(),
		ScaledBitmap.Num() * sizeof(FColor),
		PF_B8G8R8A8,
		bSetSourceData
	);

    if (Thumbnail == nullptr)
	{
		UE_LOGF(LogVariantContent, Warning, "Failed to create thumbnail texture from viewport!");
	}

    return Thumbnail;
#else
    return nullptr;
#endif
}

UTexture2D* ThumbnailGenerator::GenerateThumbnailFromObjectThumbnail(UObject* Object)
{
#if WITH_EDITOR
	if ( !Object || !GIsEditor || !GEditor )
	{
		return nullptr;
	}

    // Try to convert old thumbnails to a new thumbnail
    FName ObjectName = *Object->GetFullName();
    FThumbnailMap Map;
    ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ObjectName}, Map);

    FObjectThumbnail* OldThumbnail = Map.Find(ObjectName);
    if (OldThumbnail && !OldThumbnail->IsEmpty())
    {
        const TArray<uint8>& OldBytes = OldThumbnail->GetUncompressedImageData();

        TArray<FColor> OldColors;
        OldColors.SetNumUninitialized(OldBytes.Num() / sizeof(FColor));
        FMemory::Memcpy(OldColors.GetData(), OldBytes.GetData(), OldBytes.Num());

        // Resize if needed
        int32 Width = OldThumbnail->GetImageWidth();
        int32 Height = OldThumbnail->GetImageHeight();
        if (Width != VARIANT_MANAGER_THUMBNAIL_SIZE || Height != VARIANT_MANAGER_THUMBNAIL_SIZE)
        {
            TArray<FColor> ResizedColors;
            ResizedColors.SetNum(VARIANT_MANAGER_THUMBNAIL_SIZE * VARIANT_MANAGER_THUMBNAIL_SIZE);

			const bool bLinearSpace = true;
			const bool bForceOpaqueOutput = false;
			FImageUtils::ImageResize( Width, Height, OldColors, VARIANT_MANAGER_THUMBNAIL_SIZE, VARIANT_MANAGER_THUMBNAIL_SIZE, ResizedColors, bLinearSpace, bForceOpaqueOutput );

            OldColors = MoveTemp(ResizedColors);
        }

        FCreateTexture2DParameters Params;
        Params.bDeferCompression = true;

		const bool bSetSourceData = true;
		UTexture2D* Thumbnail = ThumbnailGeneratorImpl::CreateTextureFromBulkData(
			VARIANT_MANAGER_THUMBNAIL_SIZE,
			VARIANT_MANAGER_THUMBNAIL_SIZE,
			(void*)OldColors.GetData(),
			OldColors.Num() * sizeof(FColor),
			PF_B8G8R8A8,
			bSetSourceData
		);

		if (Thumbnail == nullptr)
		{
			UE_LOGF(LogVariantContent, Warning, "Failed to create thumbnail texture from object '%ls'!", *Object->GetName());
		}

        return Thumbnail;
    }
#endif
    return nullptr;
}
