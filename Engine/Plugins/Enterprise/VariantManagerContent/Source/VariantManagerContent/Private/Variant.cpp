// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variant.h"

#include "LevelVariantSets.h"
#include "PropertyValue.h"
#include "ThumbnailGenerator.h"
#include "VariantManagerContentLog.h"
#include "VariantManagerObjectVersion.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"

#include "Engine/Texture2D.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Variant)

#define LOCTEXT_NAMESPACE "Variant"

UVariant::FOnVariantChanged UVariant::OnThumbnailUpdated;

UVariant::FOnVariantChanged UVariant::OnDependenciesUpdated;

struct FVariantImpl
{
	static bool IsValidDependencyRecursive(const UVariant* This, const UVariant* Other, TSet<const UVariant*>& ParentStack)
	{
		if (!Other || Other == This || ParentStack.Contains(Other))
		{
			return false;
		}

		ParentStack.Add(Other);

		for (const FVariantDependency& Dependency : Other->Dependencies)
		{
			const UVariant* OtherDependency = Dependency.Variant.Get();
			if ( OtherDependency == nullptr )
			{
				// If the dependency has no variant picked yet it won't really do anything when triggered anyway,
				// so there's no reason we need to block this.
				continue;
			}

			if (!FVariantImpl::IsValidDependencyRecursive(This, OtherDependency, ParentStack))
			{
				return false;
			}
		}

		ParentStack.Remove(Other);

		return true;
	}
};

UVariant::UVariant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayText = FText::FromString(TEXT("Variant"));
#if WITH_EDITOR
	bTriedRestoringOldThumbnail = false;
#endif
}

UVariantSet* UVariant::GetParent()
{
	return Cast<UVariantSet>(GetOuter());
}

void UVariant::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FVariantManagerObjectVersion::GUID);
	int32 CustomVersion = Ar.CustomVer(FVariantManagerObjectVersion::GUID);

	if (CustomVersion < FVariantManagerObjectVersion::CategoryFlagsAndManualDisplayText)
	{
		// Recover name from back when it was an UPROPERTY
		if (Ar.IsLoading())
		{
			if (!DisplayText_DEPRECATED.IsEmpty())
			{
				DisplayText = DisplayText_DEPRECATED;
				DisplayText_DEPRECATED = FText();
			}
		}
	}
	else
	{
		Ar << DisplayText;
	}
}

void UVariant::SetDisplayText(const FText& NewDisplayText)
{
	Modify();

	DisplayText = NewDisplayText;
}

FText UVariant::GetDisplayText() const
{
	return DisplayText;
}

void UVariant::AddBindings(const TArray<UVariantObjectBinding*>& NewBindings, int32 Index)
{
	Modify();

	if (Index == INDEX_NONE)
	{
		Index = ObjectBindings.Num();
	}

	// Inserting first ensures we preserve the target order
	ObjectBindings.Insert(NewBindings, Index);

	bool bIsMoveOperation = false;
	TSet<UVariant*> ParentsModified;
	for (UVariantObjectBinding* NewBinding : NewBindings)
	{
		if (NewBinding == nullptr)
		{
			continue;
		}

		UVariant* OldParent = NewBinding->GetParent();

		if (OldParent)
		{
			if (OldParent != this)
			{
				if (!ParentsModified.Contains(OldParent))
				{
					OldParent->Modify();
					ParentsModified.Add(OldParent);
				}
				OldParent->ObjectBindings.RemoveSingle(NewBinding);
			}
			else
			{
				bIsMoveOperation = true;
			}
		}

		NewBinding->Modify();
		NewBinding->Rename(nullptr, this, REN_DontCreateRedirectors);
	}

	// If it's a move operation, we'll have to manually clear the old pointers from the array
	if (!bIsMoveOperation)
	{
		return;
	}

	TSet<FString> NewBindingPaths = TSet<FString>();
	for (UVariantObjectBinding* NewBinding : NewBindings)
	{
		NewBindingPaths.Add(NewBinding->GetObjectPath());
	}

	// Sweep back from insertion point nulling old bindings with the same path
	for (int32 SweepIndex = Index-1; SweepIndex >= 0; SweepIndex--)
	{
		if (NewBindingPaths.Contains(ObjectBindings[SweepIndex]->GetObjectPath()))
		{
			ObjectBindings[SweepIndex] = nullptr;
		}
	}
	// Sweep forward from the end of the inserted segment nulling old bindings with the same path
	for (int32 SweepIndex = Index + NewBindings.Num(); SweepIndex < ObjectBindings.Num(); SweepIndex++)
	{
		if (NewBindingPaths.Contains(ObjectBindings[SweepIndex]->GetObjectPath()))
		{
			ObjectBindings[SweepIndex] = nullptr;
		}
	}

	// Finally remove null entries
	for (int32 IterIndex = ObjectBindings.Num() - 1; IterIndex >= 0; IterIndex--)
	{
		if (ObjectBindings[IterIndex] == nullptr)
		{
			ObjectBindings.RemoveAt(IterIndex);
		}
	}
}

int32 UVariant::GetBindingIndex(UVariantObjectBinding* Binding)
{
	if (Binding == nullptr)
	{
		return INDEX_NONE;
	}

	return ObjectBindings.Find(Binding);
}

const TArray<UVariantObjectBinding*>& UVariant::GetBindings() const
{
	return ObjectBindings;
}

void UVariant::RemoveBindings(const TArray<UVariantObjectBinding*>& Bindings)
{
	Modify();

	for (UVariantObjectBinding* Binding : Bindings)
	{
		ObjectBindings.RemoveSingle(Binding);
	}
}

int32 UVariant::GetNumActors()
{
	return ObjectBindings.Num();
}

AActor* UVariant::GetActor(int32 ActorIndex)
{
	if (ObjectBindings.IsValidIndex(ActorIndex))
	{
		UVariantObjectBinding* Binding = ObjectBindings[ActorIndex];
		UObject* Obj = Binding->GetObject();
		if (AActor* Actor = Cast<AActor>(Obj))
		{
			return Actor;
		}
	}

	return nullptr;
}

UVariantObjectBinding* UVariant::GetBindingByName(const FString& ActorName)
{
	TObjectPtr<UVariantObjectBinding>* FoundBindingPtr = ObjectBindings.FindByPredicate([&ActorName](const UVariantObjectBinding* Binding)
	{
		UObject* ThisActor = Binding->GetObject();
		return ThisActor && ThisActor->GetName() == ActorName;
	});

	if (FoundBindingPtr)
	{
		return *FoundBindingPtr;
	}

	return nullptr;
}

void UVariant::SwitchOn()
{
	if (GetPackage() == GetTransientPackage())
	{
		return;
	}

	for ( const FVariantDependency& Dependency : Dependencies )
	{
		if ( !Dependency.bEnabled )
		{
			continue;
		}

		if ( UVariant* Variant = Dependency.Variant.Get() )
		{
			Variant->SwitchOn();
		}
	}

	for (UVariantObjectBinding* Binding : ObjectBindings)
	{
		for (UPropertyValue* PropCapture : Binding->GetCapturedProperties())
		{
			PropCapture->ApplyDataToResolvedObject();
		}

		Binding->ExecuteAllTargetFunctions();
	}
}

bool UVariant::IsActive()
{
	if (ObjectBindings.Num() == 0)
	{
		return false;
	}

	for (UVariantObjectBinding* Binding : ObjectBindings)
	{
		for (UPropertyValue* PropCapture : Binding->GetCapturedProperties())
		{
			if (!PropCapture->IsRecordedDataCurrent())
			{
				return false;
			}
		}
	}

	return true;
}

void UVariant::SetThumbnailFromObject(UObject* ObjectThumbnail)
{
	if (ObjectThumbnail == nullptr)
	{
		SetThumbnailInternal(nullptr);
	}
	else
	{
		if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromObjectThumbnail(ObjectThumbnail))
		{
			SetThumbnailInternal(NewThumbnail);
		}
	}
}

void UVariant::SetThumbnailFromTexture(UTexture2D* Texture)
{
	if (Texture == nullptr)
	{
		SetThumbnailInternal(nullptr);
	}
	else
	{
		if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromTexture(Texture))
		{
			SetThumbnailInternal(NewThumbnail);
		}
	}
}

void UVariant::SetThumbnailFromFile(FString FilePath)
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromFile(FilePath))
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

void UVariant::SetThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees, float MinZ, float Gamma)
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromCamera(WorldContextObject, CameraTransform, FOVDegrees, MinZ, Gamma))
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

void UVariant::SetThumbnailFromEditorViewport()
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromEditorViewport())
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

UTexture2D* UVariant::GetThumbnail()
{
#if WITH_EDITOR
	if (Thumbnail == nullptr && !bTriedRestoringOldThumbnail)
	{
		Thumbnail = ThumbnailGenerator::GenerateThumbnailFromObjectThumbnail(this);
		bTriedRestoringOldThumbnail = true;
	}
#endif

	return Thumbnail;
}

void UVariant::SetThumbnailInternal(UTexture2D* NewThumbnail)
{
	Modify();
	Thumbnail = NewThumbnail;

	if (NewThumbnail)
	{
		// This variant will now fully own this texture. It cannot be standalone or else we won't be able to delete it
		NewThumbnail->Rename(nullptr, this);
		NewThumbnail->ClearFlags( RF_Transient | RF_Standalone );
	}

	UVariant::OnThumbnailUpdated.Broadcast(this);
}

TArray<UVariant*> UVariant::GetDependents(ULevelVariantSets* LevelVariantSets, bool bOnlyEnabledDependencies )
{
	TArray<UVariant*> Result;
	if ( !LevelVariantSets )
	{
		return Result;
	}

	for ( UVariantSet* VariantSet : LevelVariantSets->GetVariantSets() )
	{
		if ( !VariantSet )
		{
			continue;
		}

		for ( UVariant* Variant : VariantSet->GetVariants() )
		{
			if ( !Variant )
			{
				continue;
			}

			for ( FVariantDependency& Dependency : Variant->Dependencies )
			{
				UVariant* TargetVariant = Dependency.Variant.Get();
				if ( (Dependency.bEnabled || !bOnlyEnabledDependencies) && TargetVariant == this )
				{
					Result.Add(Variant);
				}
			}
		}
	}

	return Result;
}

bool UVariant::IsValidDependency(const UVariant* Other) const
{
	TSet<const UVariant*> ParentStack;
	return FVariantImpl::IsValidDependencyRecursive(this, Other, ParentStack);
}

int32 UVariant::AddDependency(FVariantDependency& Dependency)
{
	UVariant* Variant = Dependency.Variant.Get();
	if (Variant && !IsValidDependency(Variant))
	{
		UE_LOGF(LogVariantContent, Error, "Cannot add variant '%ls' as a dependency of '%ls'! Cycles were detected!", *Variant->GetDisplayText().ToString(), *GetDisplayText().ToString());
		return INDEX_NONE;
	}

	Modify();
	int32 NewIndex = Dependencies.Add(Dependency);
	UVariant::OnDependenciesUpdated.Broadcast(this);
	return NewIndex;
}

FVariantDependency& UVariant::GetDependency(int32 Index)
{
	return Dependencies[Index];
}

void UVariant::SetDependency(int32 Index, FVariantDependency& Dependency)
{
	if (Dependencies.IsValidIndex(Index))
	{
		UVariant* Variant = Dependency.Variant.Get();
		if (Variant && !IsValidDependency(Variant))
		{
			UE_LOGF(LogVariantContent, Error, "Cannot set variant '%ls' as a dependency of '%ls'! Cycles were detected!",
				*Variant->GetDisplayText().ToString(),
				*GetDisplayText().ToString());
			return;
		}

		Modify();
		Dependencies[Index] = Dependency;
		UVariant::OnDependenciesUpdated.Broadcast(this);
	}
	else
	{
		UE_LOGF(LogVariantContent, Error, "Invalid dependency index '%d'! Note: Variant '%ls' has '%d' dependencies", Index, *GetDisplayText().ToString(), GetNumDependencies());
	}
}

void UVariant::DeleteDependency(int32 Index)
{
	if (Dependencies.IsValidIndex(Index))
	{
		Modify();
		Dependencies.RemoveAt(Index);
		UVariant::OnDependenciesUpdated.Broadcast(this);
	}
	else
	{
		UE_LOGF(LogVariantContent, Error, "Invalid dependency index '%d'! Note: Variant '%ls' has '%d' dependencies", Index, *GetDisplayText().ToString(), GetNumDependencies());
	}
}

int32 UVariant::GetNumDependencies()
{
	return Dependencies.Num();
}

#undef LOCTEXT_NAMESPACE
