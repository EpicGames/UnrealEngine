// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerContentModule.h"


#define LOCTEXT_NAMESPACE "VariantManagerContentModule"


class FVariantManagerContentModule : public IVariantManagerContentModule
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FVariantManagerContentModule, VariantManagerContent);

#undef LOCTEXT_NAMESPACE
