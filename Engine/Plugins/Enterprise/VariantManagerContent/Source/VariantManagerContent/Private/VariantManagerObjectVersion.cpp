// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerObjectVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FVariantManagerObjectVersion::GUID(0x24BB7AF3, 0x56464F83, 0x1F2F2DC2, 0x49AD96FF);

// Register the custom version with core
FCustomVersionRegistration GRegisterVariantManagerCustomVersion(FVariantManagerObjectVersion::GUID, FVariantManagerObjectVersion::LatestVersion, TEXT("VariantManagerObjectVer"));