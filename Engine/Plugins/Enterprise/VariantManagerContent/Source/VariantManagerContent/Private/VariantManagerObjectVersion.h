// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes to variant manager objects
struct FVariantManagerObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.21
		BeforeCustomVersionWasAdded = 0,

		CorrectSerializationOfFNameBytes,

		CategoryFlagsAndManualDisplayText,

		CorrectSerializationOfFStringBytes,

		SerializePropertiesAsNames,

		StoreDisplayOrder,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FVariantManagerObjectVersion() {}
};
