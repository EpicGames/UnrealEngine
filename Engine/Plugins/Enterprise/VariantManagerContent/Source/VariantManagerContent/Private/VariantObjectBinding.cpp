// Copyright Epic Games, Inc. All Rights Reserved. 

#include "VariantObjectBinding.h"
#include "UObject/UnrealType.h" 
#include "LevelVariantSets.h" 
#include "PropertyValue.h" 
#include "Variant.h"
#include "VariantManagerContentLog.h"
#include "VariantManagerObjectVersion.h"
#include "VariantSet.h"

#include "Engine/World.h"
#include "FunctionCaller.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantObjectBinding)

#if WITH_EDITORONLY_DATA
#endif

#define LOCTEXT_NAMESPACE "VariantObjectBinding"

UVariantObjectBinding::UVariantObjectBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UVariantObjectBinding::SetObject(UObject* InObject)
{
	Modify();

	ObjectPtr = InObject;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LazyObjectPtr_DEPRECATED = InObject;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UVariant* UVariantObjectBinding::GetParent()
{
	return Cast<UVariant>(GetOuter());
}

void UVariantObjectBinding::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FVariantManagerObjectVersion::GUID);
	int32 CustomVersion = Ar.CustomVer(FVariantManagerObjectVersion::GUID);

	if(Ar.IsLoading())
	{
		if (CustomVersion < FVariantManagerObjectVersion::StoreDisplayOrder)
		{
#if WITH_EDITORONLY_DATA
			// PropertyValue and FunctionCallers won't have any display order. Assign them
			// increasing values with FunctionCallers at the bottom
			uint32 DisplayOrder = 0;
			for (UPropertyValue* Property : CapturedProperties)
			{
				Property->SetDisplayOrder(++DisplayOrder);
			}
			for (FFunctionCaller& FunctionCaller : FunctionCallers)
			{
				FunctionCaller.SetDisplayOrder(++DisplayOrder);
			}
#endif
		}
	}
}

FText UVariantObjectBinding::GetDisplayText() const
{
	AActor* Actor = Cast<AActor>(GetObject());
	if (Actor)
	{
#if WITH_EDITOR
		const FString& Label = Actor->GetActorLabel();
#else
		const FString& Label = Actor->GetName();
#endif
		CachedActorLabel = Label;
		return FText::FromString(Label);
	}

	if (!CachedActorLabel.IsEmpty())
	{
		return FText::FromString(CachedActorLabel);
	}

	return FText::FromString(TEXT("<Unloaded binding>"));
}

FString UVariantObjectBinding::GetObjectPath() const
{
	return ObjectPtr.ToString();
}

UObject* UVariantObjectBinding::GetObject() const
{
	if (ObjectPtr.IsValid())
	{
		FSoftObjectPath TempPtr = ObjectPtr;

		// Fixup for PIE
		// We can't just call FixupForPIE blindly, and need all this structure in LVS
		// (that is, GetWorldContext and so on) because if this function is called from anything
		// that originates from a Slate tick it will occur at a moment when GPlayInEditorID is -1
		// (i.e. we're not evaluating any particular world).
		// We use the same GetWorldContext trick that LevelSequencePlaybackContext uses to go
		// through this.
		//
		// We also need to do this every time (instead of the LVS updating US) to minimize the
		// cost of having each LVS asset subscribed to editor events. Right now those event callbacks
		// just null a single pointer, which is acceptable. Having it iterate over all bindings to
		// fixup all softobjectpaths is not. On top of that, this is more efficient as it only
		// updates the required bindings on demand. In the future we can change it so that Slate
		// is not constantly calling this function every frame to repaint the node names, but keeping
		// a cached name would cause its own set of problems (currently we update the property list
		// when the name changes, so as to track objects going into/out of resolved states)
#if WITH_EDITOR
		ULevelVariantSets* LVS = GetTypedOuter<ULevelVariantSets>();
		if (LVS)
		{
			int32 PIEInstanceID;
			UWorld* World = LVS->GetWorldContext(PIEInstanceID);

			if (PIEInstanceID != INDEX_NONE)
			{
				TempPtr.FixupForPIE(PIEInstanceID);
			}
		}
#endif

		UObject* Obj = TempPtr.ResolveObject();
		if (IsValid(Obj) && !Obj->IsUnreachable())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			LazyObjectPtr_DEPRECATED = Obj;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			return Obj;
		}
		// Fixup for redirectors (e.g. when going from temp level to a saved level)
		// I could do ObjectPtr.PreSavePath, which in fact follows redirectors. This doesn't work
		// for saving after moving to a new level and then reloading, as the redirector will
		// only be created AFTER we saved. The LazyObjectPtr_DEPRECATED successfully manages to track
		// the object across levels, however. We don't exclusively use this because it is not meant
		// to update to the duplicated objects when going into PIE
		// This could potentially be enclosed in a #if WITH_EDITOR block
		// Keeping this deprecated part as a stop gap for that edge case until we remove the LazyObjectPtr code
		// At that time a FUniversalObjectLocator style approach might be the alternative
		else
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UObject* LazyObject = LazyObjectPtr_DEPRECATED.Get();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (LazyObject)
			{
				//UE_LOGF(LogVariantContent, Log, "Actor '%ls' switched path. Binding updating from path '%ls' to '%ls'", *LazyObject->GetName(), *ObjectPtr.ToString(), *LazyObject->GetFullName());
				ObjectPtr = LazyObject;
				return LazyObject;
			}
		}
	}

	return nullptr;
}

void UVariantObjectBinding::AddCapturedProperties(const TArray<UPropertyValue*>& NewProperties)
{
	Modify();

	TSet<FString> ExistingProperties;
	for (UPropertyValue* Prop : CapturedProperties)
	{
		ExistingProperties.Add(Prop->GetFullDisplayString());
	}

#if WITH_EDITORONLY_DATA
	uint32 MaxDisplayOrder = 0;
	for (UPropertyValue* NewProp : CapturedProperties)
	{
		MaxDisplayOrder = FMath::Max(MaxDisplayOrder, NewProp->GetDisplayOrder());
	}
#endif

	bool bIsMoveOperation = false;
	TSet<UVariantObjectBinding*> ParentsModified;
	for (UPropertyValue* NewProp : NewProperties)
	{
		if (NewProp == nullptr)
		{
			continue;
		}

		if (ExistingProperties.Contains(NewProp->GetFullDisplayString()))
		{
			continue;
		}

		NewProp->Modify();
		NewProp->Rename(nullptr, this, REN_DontCreateRedirectors);  // Make us its Outer

		CapturedProperties.Add(NewProp);

#if WITH_EDITORONLY_DATA
		NewProp->SetDisplayOrder(++MaxDisplayOrder);
#endif
	}

	SortCapturedProperties();
}

const TArray<UPropertyValue*>& UVariantObjectBinding::GetCapturedProperties() const
{
	return CapturedProperties;
}

void UVariantObjectBinding::RemoveCapturedProperties(const TArray<UPropertyValue*>& Properties)
{
	Modify();

	for (UPropertyValue* Prop : Properties)
	{
		CapturedProperties.RemoveSingle(Prop);
	}

	SortCapturedProperties();
}

void UVariantObjectBinding::SortCapturedProperties()
{
#if WITH_EDITORONLY_DATA
	CapturedProperties.Sort([](const UPropertyValue& A, const UPropertyValue& B)
	{
		uint32 OrderA = A.GetDisplayOrder();
		uint32 OrderB = B.GetDisplayOrder();

		if (OrderA == OrderB)
		{
		return A.GetFullDisplayString() < B.GetFullDisplayString();
		}

		return OrderA < OrderB;
	});
#endif
}

void UVariantObjectBinding::AddFunctionCallers(const TArray<FFunctionCaller>& InFunctionCallers)
{
	Modify();

	FunctionCallers.Append(InFunctionCallers);
}

TArray<FFunctionCaller>& UVariantObjectBinding::GetFunctionCallers()
{
	return FunctionCallers;
}

const TArray<FFunctionCaller>& UVariantObjectBinding::GetFunctionCallers() const
{
	return FunctionCallers;
}

void UVariantObjectBinding::RemoveFunctionCallers(const TArray<FFunctionCaller*>& InFunctionCallers)
{
	Modify();

#if WITH_EDITORONLY_DATA
	// It's ok that we're passing pointers everywhere since the ultimate consumer of the "remove function callers"
	// callstack is this very function, and we're the object that actually owns these FunctionCallers, so they
	// won't go out of scope untill we touch them
	FunctionCallers.RemoveAll([&](const FFunctionCaller& Item)
	{
		return InFunctionCallers.Contains(&Item);
	});
#endif
}

namespace
{
	FObjectProperty* IsNonRefObjectParamNamed(FProperty* Prop, FName ExpectedName)
	{ 
		FObjectProperty* ObjProp = CastField<FObjectProperty>(Prop); 
		if (!ObjProp)
		{
			return nullptr;
		} 
		if (ObjProp->GetFName() != ExpectedName)
		{
			return nullptr;
		} 
		if ((ObjProp->GetPropertyFlags() & CPF_ReferenceParm) != 0)
		{
			return nullptr;
		} 
		return ObjProp; 
	}; 
	
	bool IsNonRefNameToStringMapParam(FProperty* Prop, FName ExpectedName)
	{
		FMapProperty* MapProp = CastField<FMapProperty>(Prop); 
		if (!MapProp)
		{
			return false;
		} 
		if (MapProp->GetFName() != ExpectedName)
		{
			return false;
		} 
		if ((MapProp->GetPropertyFlags() & CPF_ReferenceParm) != 0)
		{
			return false;
		} 
		
		const FNameProperty* KeyProp = CastField<FNameProperty>(MapProp->KeyProp);
		const FStrProperty* ValueProp = CastField<FStrProperty>(MapProp->ValueProp);

		return (KeyProp != nullptr) && (ValueProp != nullptr);
	}; 
		
	bool ValidateTwoParamSignature(UFunction* InFunc, UClass*& OutExpectedTargetClass)
	{
		OutExpectedTargetClass = nullptr; 
		FProperty* P0 = InFunc ? InFunc->PropertyLink : nullptr; 
		FProperty* P1 = P0 ? P0->PropertyLinkNext : nullptr; 
		FObjectProperty* TargetProp = IsNonRefObjectParamNamed(P0, TARGET_PIN_NAME); 
		if (!TargetProp || !P1 || P1->PropertyLinkNext)
		{
			return false;
		} 
		if (!IsNonRefNameToStringMapParam(P1, TEXT("Arguments")))
		{
			return false;
		} 
		OutExpectedTargetClass = TargetProp->PropertyClass; 
		return true;
	}; 
	
	bool ValidateFiveParamSignature(UFunction* InFunc, UClass*& OutExpectedTargetClass)
	{
		OutExpectedTargetClass = nullptr;
		FProperty* P0 = InFunc ? InFunc->PropertyLink : nullptr;
		FProperty* P1 = P0 ? P0->PropertyLinkNext : nullptr;
		FProperty* P2 = P1 ? P1->PropertyLinkNext : nullptr;
		FProperty* P3 = P2 ? P2->PropertyLinkNext : nullptr;
		FProperty* P4 = P3 ? P3->PropertyLinkNext : nullptr;
		FObjectProperty* TargetProp = IsNonRefObjectParamNamed(P0, TARGET_PIN_NAME);
		const FObjectProperty* LVSProp = CastField<FObjectProperty>(P1);
		const FObjectProperty* VSProp = CastField<FObjectProperty>(P2);
		const FObjectProperty* VProp = CastField<FObjectProperty>(P3);
		if (!TargetProp || !LVSProp || !VSProp || !VProp || !P4 || P4->PropertyLinkNext)
		{
			return false;
		} 
		if ((LVSProp->GetPropertyFlags() & CPF_ReferenceParm) != 0 || 
			(LVSProp->PropertyClass != ULevelVariantSets::StaticClass()))
		{
			return false;
		} 
		if ((VSProp->GetPropertyFlags() & CPF_ReferenceParm) != 0 || 
			(VSProp->PropertyClass != UVariantSet::StaticClass()))
		{
			return false;
		}
		if ((VProp->GetPropertyFlags() & CPF_ReferenceParm) != 0 || 
			(VProp->PropertyClass != UVariant::StaticClass()))
		{
			return false;
		}
		if (!IsNonRefNameToStringMapParam(P4, TEXT("Arguments")))
		{
			return false;
		}
		OutExpectedTargetClass = TargetProp->PropertyClass; 
		return true;
	};

}

void UVariantObjectBinding::ExecuteTargetFunction(FName FunctionName)
{
	ULevelVariantSets* ParentLVS = GetTypedOuter<ULevelVariantSets>();
	if (!ParentLVS)
	{
		return;
	}

	UObject* BoundObject = GetObject();
	if (!BoundObject)
	{
		return;
	}

	UObject* DirectorInstance = ParentLVS->GetDirectorInstance(BoundObject);
	if (!DirectorInstance)
	{
		return;
	}

	UFunction* Func = DirectorInstance->FindFunction(FunctionName);
	if (!Func)
	{
		return;
	}

	//need to check if we're in edit mode and the function is CallInEditor
#if WITH_EDITOR
	const static FName NAME_CallInEditor(TEXT("CallInEditor"));

	UWorld* World = DirectorInstance->GetWorld();

	if (World == nullptr)
	{
		UE_LOGF(LogVariantContent, Warning, "Cannot call function '%ls' as Director Instance's World is null.", *FunctionName.ToString());
		return;
	}

	if (World->WorldType == EWorldType::Editor && !Func->HasMetaData(NAME_CallInEditor))
	{
		UE_LOGF(LogVariantContent, Warning, "Cannot call function '%ls' as it doesn't have the CallInEditor option checked! Also note that calling this from the editor may have irreversible effects on the level.", *FunctionName.ToString());
		return;
	}
#endif

	if (Func->NumParms == 0)
	{
		DirectorInstance->ProcessEvent(Func, nullptr);
	}
	// Mostly for backwards compatibility
	else if (Func->NumParms == 1)
	{
		if (FObjectProperty* ObjectParameter = CastField<FObjectProperty>(Func->PropertyLink))
		{
			if (!ObjectParameter->PropertyClass || BoundObject->IsA(ObjectParameter->PropertyClass))
			{
				DirectorInstance->ProcessEvent(Func, &BoundObject);
			}
			else
			{
				UE_LOGF(LogVariantContent, Error, "Failed to call function '%ls' with object '%ls' because it is not the correct type. Function expects a '%ls' but target object is a '%ls'.",
					*Func->GetName(),
					*BoundObject->GetName(),
					*ObjectParameter->PropertyClass->GetName(),
					*BoundObject->GetClass()->GetName()
				);
			}
		}
	}
	else if (Func->NumParms == 2)
	{
		UClass* ExpectedClass = nullptr;
		if (!ValidateTwoParamSignature(Func, ExpectedClass))
		{
			UE_LOGF(LogVariantContent, Error, "Function '%ls' has NumParms==2 but does not match expected signature (Target, Arguments map).", *Func->GetName());
			return;
		} 

		if (ExpectedClass && !BoundObject->IsA(ExpectedClass))
		{
			UE_LOGF(LogVariantContent, Error, "Failed to call function '%ls' with object '%ls' because it is not the correct type. Function expects a '%ls' but target object is a '%ls'.", *Func->GetName(), *BoundObject->GetName(), *ExpectedClass->GetName(), *BoundObject->GetClass()->GetName());
			return;
		} 

		struct
		{
			UObject* Target = nullptr;
			TMap<FName, FString> Arguments;
		} FuncParams;
		FuncParams.Target = BoundObject;

		if (const FFunctionCaller* FunctionCaller = FunctionCallers.FindByPredicate([FunctionName](const FFunctionCaller& FunctionCaller) { return FunctionCaller.FunctionName == FunctionName; }))
		{
			FuncParams.Arguments = FunctionCaller->FunctionArguments;
		}
		
		DirectorInstance->ProcessEvent(Func, &FuncParams);
	}
	else if (Func->NumParms == 4)
	{
		// Setup parameters to the Func as a struct
		struct
		{
			UObject* Target = nullptr;
			ULevelVariantSets* LevelVariantSets = nullptr;
			UVariantSet* VariantSet = nullptr;
			UVariant* Variant = nullptr;
		} FuncParams;
		FuncParams.Target = BoundObject;
		FuncParams.Variant = GetParent();
		FuncParams.VariantSet = FuncParams.Variant ? FuncParams.Variant->GetParent() : nullptr;
		FuncParams.LevelVariantSets = FuncParams.VariantSet ? FuncParams.VariantSet->GetParent() : nullptr;

		// Check if our bound object is of the right class for this function (that class could be StaticMeshActor/LightActor/etc.)
		bool bValidClass = false;
		UClass* ExpectedClass = nullptr;
		for (TFieldIterator<FObjectProperty> Iter(Func); Iter; ++Iter)
		{
			FObjectProperty* Parameter = *Iter;
			if (!Parameter)
			{
				continue;
			}

			if (Parameter->GetFName() == TARGET_PIN_NAME && (Parameter->GetPropertyFlags() & CPF_ReferenceParm) == 0)
			{
				ExpectedClass = Parameter->PropertyClass;
				if (BoundObject->IsA(ExpectedClass))
				{
					bValidClass = true;
				}
				break;
			}
		}

		if (bValidClass)
		{
			DirectorInstance->ProcessEvent(Func, &FuncParams);
		}
		else
		{
			UE_LOGF(LogVariantContent, Error, "Failed to call function '%ls' with object '%ls' because it is not the correct type. Function expects a '%ls' but target object is a '%ls'.",
				*Func->GetName(),
				*BoundObject->GetName(),
				ExpectedClass ? *ExpectedClass->GetName() : TEXT("nullptr"),
				*BoundObject->GetClass()->GetName()
			);
		}
	}
	else if (Func->NumParms == 5)
	{
		UClass* ExpectedClass = nullptr;
		if (!ValidateFiveParamSignature(Func, ExpectedClass))
		{
			UE_LOGF(LogVariantContent, Error, "Function '%ls' has NumParms==5 but does not match expected signature (Target, LVS, VariantSet, Variant, Arguments map).", *Func->GetName());
			return;
		}
		if (ExpectedClass && !BoundObject->IsA(ExpectedClass))
		{
			UE_LOGF(LogVariantContent, Error, "Failed to call function '%ls' with object '%ls' because it is not the correct type. Function expects a '%ls' but target object is a '%ls'.", 
				*Func->GetName(), 
				*BoundObject->GetName(),
				*ExpectedClass->GetName(),
				*BoundObject->GetClass()->GetName());
			return;
		} 
		// Setup parameters to the Func as a struct
		struct
		{
			UObject* Target = nullptr;
			ULevelVariantSets* LevelVariantSets = nullptr;
			UVariantSet* VariantSet = nullptr;
			UVariant* Variant = nullptr;
			TMap<FName, FString> Arguments;
		} FuncParams;
		FuncParams.Target = BoundObject;
		FuncParams.Variant = GetParent();
		FuncParams.VariantSet = FuncParams.Variant ? FuncParams.Variant->GetParent() : nullptr;
		FuncParams.LevelVariantSets = FuncParams.VariantSet ? FuncParams.VariantSet->GetParent() : nullptr;
		if (const FFunctionCaller* FunctionCaller = FunctionCallers.FindByPredicate([FunctionName](const FFunctionCaller& FunctionCaller) { return FunctionCaller.FunctionName == FunctionName; }))
		{
			FuncParams.Arguments = FunctionCaller->FunctionArguments;
		}
		DirectorInstance->ProcessEvent(Func, &FuncParams);
	}
}

void UVariantObjectBinding::ExecuteAllTargetFunctions()
{
	for (FFunctionCaller& Caller : FunctionCallers)
	{
		ExecuteTargetFunction(Caller.FunctionName);
	}
}

#if WITH_EDITORONLY_DATA
void UVariantObjectBinding::UpdateFunctionCallerNames()
{
	ULevelVariantSets* ParentLVS = GetTypedOuter<ULevelVariantSets>();
	UObject* DirectorInstance = ParentLVS ? ParentLVS->GetDirectorInstance(GetObject()) : nullptr;
	if (!DirectorInstance)
	{
		return;
	}

	bool bHasChanged = false;

	for (FFunctionCaller& Caller : FunctionCallers)
	{
		FName OldFunctionName = Caller.FunctionName;

		Caller.CacheFunctionName();

		// Catch case where function has been deleted and clear the caller,
		// as the entry node will still be valid
		UFunction* Func = DirectorInstance->FindFunction(Caller.FunctionName);
		if (!Func)
		{
			Caller.SetFunctionEntry(nullptr);
		}

		if (Caller.FunctionName != OldFunctionName)
		{
			bHasChanged = true;
		}
	}

	if (bHasChanged)
	{
		MarkPackageDirty();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
