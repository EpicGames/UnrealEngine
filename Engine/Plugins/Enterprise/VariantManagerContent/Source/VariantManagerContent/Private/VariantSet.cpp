// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantSet.h"

#include "LevelVariantSets.h"
#include "ThumbnailGenerator.h"
#include "UObject/Package.h"
#include "Variant.h"
#include "VariantManagerObjectVersion.h"

#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantSet)

#define LOCTEXT_NAMESPACE "VariantManagerVariantSet"

UVariantSet::FOnVariantSetChanged UVariantSet::OnThumbnailUpdated;

namespace VariantSetImpl
{
	/** Makes it so that all others variants that depend on 'Variant' have those particular dependencies deleted */
	void ResetVariantDependents( UVariant* Variant )
	{
		if ( !Variant )
		{
			return;
		}

		ULevelVariantSets* LevelVariantSets = Variant->GetTypedOuter<ULevelVariantSets>();
		if ( !LevelVariantSets )
		{
			return;
		}

		// Reset dependencies if we're being removed
		const bool bOnlyEnabledDependencies = false;
		for ( UVariant* Dependent : Variant->GetDependents( LevelVariantSets, bOnlyEnabledDependencies ) )
		{
			for ( int32 DependencyIndex = Dependent->GetNumDependencies() - 1; DependencyIndex >= 0; --DependencyIndex )
			{
				FVariantDependency& Dependency = Dependent->GetDependency( DependencyIndex );
				UVariant* TargetVariant = Dependency.Variant.Get();
				if ( TargetVariant == Variant )
				{
					// Delete the entire dependency because we can't leave a dependency without a valid Variant selected or
					// we may run into minor slightly awkward states (i.e. not being able to pick *any* variant)
					Dependent->DeleteDependency( DependencyIndex );
				}
			}
		}
	}
}

UVariantSet::UVariantSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayText = FText::FromString(TEXT("Variant Set"));
	bExpanded = true;
}

ULevelVariantSets* UVariantSet::GetParent()
{
	return Cast<ULevelVariantSets>(GetOuter());
}

void UVariantSet::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FVariantManagerObjectVersion::GUID);
	int32 CustomVersion = Ar.CustomVer(FVariantManagerObjectVersion::GUID);

	if (CustomVersion < FVariantManagerObjectVersion::CategoryFlagsAndManualDisplayText)
	{
		// Recover name from back when it was an UPROPERTY
		if (Ar.IsLoading())
		{
			if (!DisplayText_DEPRECATED.IsEmpty())
			{
				DisplayText = DisplayText_DEPRECATED;
				DisplayText_DEPRECATED = FText();
			}
		}
	}
	else
	{
		Ar << DisplayText;
	}
}

bool UVariantSet::IsExpanded() const
{
	return bExpanded;
}

void UVariantSet::SetExpanded(bool bInExpanded)
{
	bExpanded = bInExpanded;
}

void UVariantSet::SetDisplayText(const FText& NewDisplayText)
{
	Modify();

	DisplayText = NewDisplayText;
}

FText UVariantSet::GetDisplayText() const
{
	return DisplayText;
}

FString UVariantSet::GetUniqueVariantName(const FString& InPrefix) const
{
	TSet<FString> UniqueNames;
	for (UVariant* Variant : Variants)
	{
		UniqueNames.Add(Variant->GetDisplayText().ToString());
	}

	if (!UniqueNames.Contains(InPrefix))
	{
		return InPrefix;
	}

	FString VarName = FString(InPrefix);

	// Remove potentially existing suffix numbers
	FString LastChar = VarName.Right(1);
	while (LastChar.IsNumeric())
	{
		VarName.LeftChopInline(1, EAllowShrinking::No);
		LastChar = VarName.Right(1);
	}

	// Add a numbered suffix
	if (UniqueNames.Contains(VarName) || VarName.IsEmpty())
	{
		int32 Suffix = 0;
		while (UniqueNames.Contains(VarName + FString::FromInt(Suffix)))
		{
			Suffix += 1;
		}

		VarName = VarName + FString::FromInt(Suffix);
	}

	return VarName;
}

void UVariantSet::AddVariants(const TArray<UVariant*>& NewVariants, int32 Index)
{
	Modify();

	if (Index == INDEX_NONE)
	{
		Index = Variants.Num();
	}

	TSet<FString> OldNames;
	for (UVariant* Var : Variants)
	{
		OldNames.Add(Var->GetDisplayText().ToString());
	}

	// Inserting first ensures we preserve the target order
	Variants.Insert(NewVariants, Index);

	bool bIsMoveOperation = false;
	TSet<UVariantSet*> ParentsModified;
	for (UVariant* NewVariant : NewVariants)
	{
		if (NewVariant == nullptr)
		{
			continue;
		}

		UVariantSet* OldParent = NewVariant->GetParent();

		// We can't just RemoveBinding since that might remove the wrong item
		if (OldParent)
		{
			if (OldParent != this)
			{
				if (!ParentsModified.Contains(OldParent))
				{
					OldParent->Modify();
					ParentsModified.Add(OldParent);
				}
				OldParent->Variants.RemoveSingle(NewVariant);
				VariantSetImpl::ResetVariantDependents(NewVariant);
			}
			else
			{
				bIsMoveOperation = true;
			}
		}

		NewVariant->Modify();
		NewVariant->Rename(nullptr, this, REN_DontCreateRedirectors);  // Change parents

		// Update name if we're from a different parent but our names collide
		FString IncomingName = NewVariant->GetDisplayText().ToString();
		if (OldParent != this && OldNames.Contains(IncomingName))
		{
			NewVariant->SetDisplayText(FText::FromString(GetUniqueVariantName(IncomingName)));
		}
	}

	// If it's a move operation, we'll have to manually clear the old pointers from the array
	if (bIsMoveOperation)
	{
		TSet<UVariant*> SetOfNewVariants = TSet<UVariant*>(NewVariants);

		// Sweep back from insertion point nulling old bindings with the same GUID
		for (int32 SweepIndex = Index-1; SweepIndex >= 0; SweepIndex--)
		{
			if (SetOfNewVariants.Contains(Variants[SweepIndex]))
			{
				Variants[SweepIndex] = nullptr;
			}
		}
		// Sweep forward from the end of the inserted segment nulling old bindings with the same GUID
		for (int32 SweepIndex = Index + NewVariants.Num(); SweepIndex < Variants.Num(); SweepIndex++)
		{
			if (SetOfNewVariants.Contains(Variants[SweepIndex]))
			{
				Variants[SweepIndex] = nullptr;
			}
		}

		// Finally remove null entries
		for (int32 IterIndex = Variants.Num() - 1; IterIndex >= 0; IterIndex--)
		{
			if (Variants[IterIndex] == nullptr)
			{
				Variants.RemoveAt(IterIndex);
			}
		}
	}
}

int32 UVariantSet::GetVariantIndex(UVariant* Var) const
{
	if (Var == nullptr)
	{
		return INDEX_NONE;
	}

	return Variants.Find(Var);
}

const TArray<UVariant*>& UVariantSet::GetVariants() const
{
	return Variants;
}

void UVariantSet::RemoveVariants(const TArray<UVariant*>& InVariants)
{
	Modify();

	for (UVariant* Variant : InVariants)
	{
		Variants.RemoveSingle(Variant);
		VariantSetImpl::ResetVariantDependents( Variant );
		Variant->Rename(nullptr, GetTransientPackage());
	}
}

int32 UVariantSet::GetNumVariants() const
{
	return Variants.Num();
}

UVariant* UVariantSet::GetVariant(int32 VariantIndex)
{
	if (Variants.IsValidIndex(VariantIndex))
	{
		return Variants[VariantIndex];
	}

	return nullptr;
}

UVariant* UVariantSet::GetVariantByName(FString VariantName)
{
	TObjectPtr<UVariant>* VarPtr = Variants.FindByPredicate([VariantName](const UVariant* Var)
	{
		return Var && Var->GetDisplayText().ToString() == VariantName;
	});

	if (VarPtr)
	{
		return *VarPtr;
	}
	return nullptr;
}

int32 UVariantSet::GetVariantIndexByName(FString VariantName) const
{
	return Variants.IndexOfByPredicate([VariantName](const UVariant* Var)
	{
		return Var && Var->GetDisplayText().ToString() == VariantName;
	});
}

void UVariantSet::SetThumbnailFromObject(UObject* ObjectThumbnail)
{
	if (ObjectThumbnail == nullptr)
	{
		SetThumbnailInternal(nullptr);
	}
	else
	{
		if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromObjectThumbnail(ObjectThumbnail))
		{
			SetThumbnailInternal(NewThumbnail);
		}
	}
}

void UVariantSet::SetThumbnailFromTexture(UTexture2D* Texture)
{
	if (Texture == nullptr)
	{
		SetThumbnailInternal(nullptr);
	}
	else
	{
		if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromTexture(Texture))
		{
			SetThumbnailInternal(NewThumbnail);
		}
	}
}

void UVariantSet::SetThumbnailFromFile(FString FilePath)
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromFile(FilePath))
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

void UVariantSet::SetThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees, float MinZ, float Gamma)
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromCamera(WorldContextObject, CameraTransform, FOVDegrees, MinZ, Gamma))
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

void UVariantSet::SetThumbnailFromEditorViewport()
{
	if (UTexture2D* NewThumbnail = ThumbnailGenerator::GenerateThumbnailFromEditorViewport())
	{
		SetThumbnailInternal(NewThumbnail);
	}
}

UTexture2D* UVariantSet::GetThumbnail()
{
	return Thumbnail;
}

void UVariantSet::SetThumbnailInternal(UTexture2D* NewThumbnail)
{
	Modify();

	Thumbnail = NewThumbnail;

	if (NewThumbnail)
	{
		// This variant set will now fully own this texture. It cannot be standalone or else we won't be able to delete it
		NewThumbnail->Rename(nullptr, this);
		NewThumbnail->ClearFlags( RF_Transient | RF_Standalone );
	}

	UVariantSet::OnThumbnailUpdated.Broadcast(this);
}

#undef LOCTEXT_NAMESPACE
