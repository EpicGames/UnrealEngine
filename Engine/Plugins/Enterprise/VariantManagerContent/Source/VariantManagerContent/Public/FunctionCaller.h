// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FunctionCaller.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

#define TARGET_PIN_NAME TEXT("Target")
#define LEVEL_VARIANT_SETS_PIN_NAME TEXT("Level Variant Sets")
#define VARIANT_SET_PIN_NAME TEXT("Variant Set")
#define VARIANT_PIN_NAME TEXT("Variant")
#define ARGUMENTS_PIN_NAME TEXT("Arguments")

// This file is based on MovieSceneEvent.h

class UEdGraph;
class UK2Node_FunctionEntry;
class UFunctionCallerSectionBase;

USTRUCT()
struct FFunctionCaller
{
	GENERATED_BODY()

	/**
	 * Called after this event has been serialized in order to cache the function pointer if necessary
	 */
	UE_API void PostSerialize(const FArchive& Ar);

	/**
	 * The function that should be called to invoke this event.
	 * Functions must have either:
	 * - no parameters
	 * - a single, pass-by-value object/interface parameter, with no return parameter
	 * - two parameters, one for bound object and another for arguments
	 * - four parameters: bound object, level variant set, variant set and variant
	 * - five parameters: bound object, level variant set, variant set, variant and arguments
	 */
	UPROPERTY()
	FName FunctionName;
		
	/**
	 * The function arguments to be passed when invoking this event (provided function supports arguments).
	 */
	UPROPERTY()
	TMap<FName, FString> FunctionArguments;

#if WITH_EDITORONLY_DATA

public:
	/**
	 * Get the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	UE_API uint32 GetDisplayOrder() const;

	/**
	 * Set the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	UE_API void SetDisplayOrder(uint32 InDisplayOrder);

	/**
	 * Cache the function name to call from the blueprint function entry node. Will only cache the function if it has a valid signature.
	 */
	UE_API void CacheFunctionName();

	/**
	 * Check whether this event is bound to a valid blueprint entry node
	 *
	 * @return true if this event is bound to a function entry node with a valid signature, false otherwise.
	 */
	UE_API bool IsBoundToBlueprint() const;

	/**
	 * Helper function to determine whether the specified function entry is valid for this event
	 *
	 * @param Node         The node to test
	 * @return true if the function entry node is compatible with a variant director specification, false otherwise
	 */
	static UE_API bool IsValidFunction(UK2Node_FunctionEntry* Node);

	/**
	 * Retrieve the function entry node this event is bound to
	 *
	 * @note Events may be bound to invalid function entries if they have been changed since they were assigned.
	 * @see SetFunctionEntry, IsValidFunction
	 * @return The function entry node if still available, nullptr if it has been destroyed, or was never assigned.
	 */
	UE_API UK2Node_FunctionEntry* GetFunctionEntry() const;

	/**
	 * Set the function entry that this event should trigger
	 *
	 * @param Entry        The graph node to bind to
	 */
	UE_API void SetFunctionEntry(UK2Node_FunctionEntry* Entry);

	/**
	 * Returns true if bound to a function that can take arguments
	 *
	 * @return true if bound and if function can take arguments.
	 */
	UE_API bool CanHaveArguments() const;
	
	/**
	 * Returns true if bound to a function that can take arguments
	 * 
	 * @param ThisFunc		The graph node to bind to
	 * @return true if bound and if function can take arguments.
	 */
	UE_API static bool CanHaveArguments(UK2Node_FunctionEntry* ThisFunc);

private:
	/** Weak pointer to the function entry within the blueprint graph for this event. Stored as an editor-only UObject so UHT can parse it when building for non-editor. */
	UPROPERTY()
	TWeakObjectPtr<UObject> FunctionEntry;

	UPROPERTY()
	uint32 DisplayOrder = 0;

#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FFunctionCaller> : TStructOpsTypeTraitsBase2<FFunctionCaller>
{
	enum { WithPostSerialize = true };
};

#undef UE_API
