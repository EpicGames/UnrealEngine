// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LevelVariantSets.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class ALevelVariantSetsActor;
class UBlueprint;
class UBlueprintGeneratedClass;
class ULevelVariantSetsFunctionDirector;
class UVariantSet;

UCLASS(MinimalAPI, BlueprintType)
class ULevelVariantSets : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// UObject interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject interface

	UE_API void AddVariantSets(const TArray<UVariantSet*>& NewVariantSets, int32 Index = INDEX_NONE);
	UE_API int32 GetVariantSetIndex(UVariantSet* VarSet);
	UE_API const TArray<UVariantSet*>& GetVariantSets() const;
	UE_API void RemoveVariantSets(const TArray<UVariantSet*> InVariantSets);

	UE_API FString GetUniqueVariantSetName(const FString& Prefix);

	// Return an existing or create a new director instance for the world that WorldContext is in
	UE_API UObject* GetDirectorInstance(UObject* WorldContext);

	UFUNCTION(BlueprintPure, Category="LevelVariantSets")
	UE_API int32 GetNumVariantSets();

	UFUNCTION(BlueprintPure, Category="LevelVariantSets")
	UE_API UVariantSet* GetVariantSet(int32 VariantSetIndex);

	UFUNCTION(BlueprintPure, Category="LevelVariantSets")
	UE_API UVariantSet* GetVariantSetByName(FString VariantSetName);
	
	UFUNCTION(BlueprintPure, Category="LevelVariantSets")
	UE_API int32 GetVariantSetIndex(FString VariantSetName) const;

#if WITH_EDITOR
	UE_API void SetDirectorGeneratedBlueprint(UObject* InDirectorBlueprint);
	UE_API UObject* GetDirectorGeneratedBlueprint();
	UE_API UBlueprintGeneratedClass* GetDirectorGeneratedClass();
	UE_API void OnDirectorBlueprintRecompiled(UBlueprint* InBP);

	// Returns the current world, as well as its PIEInstanceID
	// This will break when the engine starts supporting multiple, concurrent worlds
	UE_API UWorld* GetWorldContext(int32& OutPIEInstanceID);
	UE_API void ResetWorldContext();

private:

	// Called on level transitions, invalidate our CurrentWorld pointer
	void OnPieEvent(bool bIsSimulating);
	void OnMapChange(uint32 MapChangeFlags);

	// Returns the first PIE world that we find or the Editor world
	// Also outputs the PIEInstanceID of the WorldContext, for convenience,
	// which will be -1 for editor worlds
	UWorld* ComputeCurrentWorld(int32& OutPIEInstanceID);

	// Sub/unsub to PIE/Map change events, which our CurrentWorld whenever
	// something happens
	void SubscribeToEditorDelegates();
	void UnsubscribeToEditorDelegates();

	// Sub/unsub to whenever our director is recompiled, which allows us to
	// track when functions get deleted/renamed/updated
	void SubscribeToDirectorCompiled();
	void UnsubscribeToDirectorCompiled();

#endif

	// Whenever a director is destroyed we remove it from our map, so next
	// time we need it we know we have to recreate it
	void HandleDirectorDestroyed(ULevelVariantSetsFunctionDirector* Director);

private:

#if WITH_EDITORONLY_DATA
	UWorld* CurrentWorld = nullptr;
	int32 CurrentPIEInstanceID = INDEX_NONE;

	// A pointer to the director blueprint that generates this levelvariantsets' DirectorClass
	UPROPERTY()
	TObjectPtr<UObject> DirectorBlueprint;

	FDelegateHandle OnBlueprintCompiledHandle;
	FDelegateHandle EndPlayDelegateHandle;
#endif

	// The class that is used to spawn this levelvariantsets' director instance.
	// Director instances are allocated one per instance
	UPROPERTY()
	TObjectPtr<UBlueprintGeneratedClass> DirectorClass;

	UPROPERTY()
	TArray<TObjectPtr<UVariantSet>> VariantSets;

	// We keep one director instance per world to serve as world context for our function caller functions.
	// Their lifetimes are guaranteed by spawned ALevelVariantSetsActors
	TMap<UWorld*, TWeakObjectPtr<UObject>> WorldToDirectorInstance;
};

#undef UE_API
