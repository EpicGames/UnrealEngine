// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "LevelVariantSetsActor.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class UBlueprintGeneratedClass;
class ULevelVariantSets;
class ULevelVariantSetsFunctionDirector;
class UVariantSet;

UCLASS(MinimalAPI, hideCategories=(Rendering, Physics, HLOD, Activation, Input, Actor, Cooking))
class ALevelVariantSetsActor : public AActor
{
public:

	GENERATED_BODY()

	UE_API ALevelVariantSetsActor(const FObjectInitializer& Init);

	// Non-const so that it doesn't show as pure in blueprints,
	// since it might trigger a load
	UFUNCTION(BlueprintCallable, Category="LevelVariantSets", meta=(ToolTip="Returns the LevelVariantSets asset, optionally loading it if necessary"))
	UE_API ULevelVariantSets* GetLevelVariantSets(bool bLoad = false);

	UFUNCTION(BlueprintCallable, Category="LevelVariantSets")
	UE_API void SetLevelVariantSets(ULevelVariantSets* InVariantSets);

	UFUNCTION(BlueprintCallable, Category="LevelVariantSets")
	UE_API bool SwitchOnVariantByName(FString VariantSetName, FString VariantName);

	UFUNCTION(BlueprintCallable, Category="LevelVariantSets")
	UE_API bool SwitchOnVariantByIndex(int32 VariantSetIndex, int32 VariantIndex);

public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="LevelVariantSets", meta=(AllowedClasses="/Script/VariantManagerContent.LevelVariantSets"))
	FSoftObjectPath LevelVariantSets;

private:

	friend class ULevelVariantSets;

	UPROPERTY(Transient)
	TMap<TObjectPtr<UBlueprintGeneratedClass>, TObjectPtr<ULevelVariantSetsFunctionDirector>> DirectorInstances;
};

#undef UE_API
