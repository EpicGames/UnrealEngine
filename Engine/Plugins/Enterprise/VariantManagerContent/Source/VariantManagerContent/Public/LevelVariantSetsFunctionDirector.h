// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelVariantSetsFunctionDirector.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class ULevelVariantSetsFunctionDirector;

DECLARE_MULTICAST_DELEGATE_OneParam(OnDirectorDestroyed, ULevelVariantSetsFunctionDirector*);

UCLASS(MinimalAPI, Blueprintable)
class ULevelVariantSetsFunctionDirector : public UObject
{
public:
	GENERATED_BODY()

	~ULevelVariantSetsFunctionDirector() = default;

	virtual void BeginDestroy() override
	{
		OnDestroy.Broadcast(this);
		Super::BeginDestroy();
	}

	OnDirectorDestroyed& GetOnDestroy()
	{
		return OnDestroy;
	}

	UE_API virtual UWorld* GetWorld() const override;

	// Called from our destructor
	// Mainly used by levelvariantsets to keep track of when a director becomes invalid and
	// we need to create a new one for that world
	OnDirectorDestroyed OnDestroy;
};

#undef UE_API
