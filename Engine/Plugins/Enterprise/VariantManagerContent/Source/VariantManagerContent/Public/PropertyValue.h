// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PropertyValue.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

#define PATH_DELIMITER TEXT(" / ")
#define ATTACH_CHILDREN_NAME TEXT("Children")

DECLARE_MULTICAST_DELEGATE(FOnPropertyRecorded);
DECLARE_MULTICAST_DELEGATE(FOnPropertyApplied);

class UVariantObjectBinding;
class USCS_Node;
class FProperty;

UENUM()
enum class EPropertyValueCategory : uint8
{
	Undefined = 0,
	Generic = 1,
	RelativeLocation = 2,
	RelativeRotation = 4,
	RelativeScale3D = 8,
	Visibility = 16,
	Material = 32,
	Color = 64,
	Option = 128
};
ENUM_CLASS_FLAGS(EPropertyValueCategory)

// Describes one link in a full property path
// For array properties, a link might be the outer (e.g. AttachChildren, -1, None)
// while also it may be an inner (e.g. AttachChildren, 2, Cube)
// Doing this allows us to resolve components regardless of their order, which
// is important for handling component reordering and transient components (e.g.
// runtime billboard components, etc)
USTRUCT()
struct FCapturedPropSegment
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString PropertyName;

	UPROPERTY()
	int32 PropertyIndex = INDEX_NONE;

	UPROPERTY()
	FString ComponentName;
};

UCLASS(MinimalAPI, BlueprintType)
class UPropertyValue : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UE_API void Init(const TArray<FCapturedPropSegment>& InCapturedPropSegments, FFieldClass* InLeafPropertyClass, const FString& InFullDisplayString, const FName& InPropertySetterName, EPropertyValueCategory InCategory = EPropertyValueCategory::Generic);

	UE_API class UVariantObjectBinding* GetParent() const;

	// Combined hash of this property and its indices
	// We don't use GetTypeHash for this because almost always we want to hash UPropertyValues by
	// the pointer instead, for complete uniqueness even with the same propertypath
	// This is mostly just used for grouping UPropertyValues together for editing multiple at once
	UE_API uint32 GetPropertyPathHash();

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	// Tries to resolve the property value on the passed object, or the parent binding's bound object if the argument is nullptr
	UE_API virtual bool Resolve(UObject* OnObject = nullptr);
	UE_API bool HasValidResolve() const;
	UE_API void ClearLastResolve();

	UE_API void* GetPropertyParentContainerAddress() const;
	UE_API virtual UStruct* GetPropertyParentContainerClass() const;

	// Fetches the value bytes for this property from the resolved object
	UE_API virtual TArray<uint8> GetDataFromResolvedObject() const;

	// Uses GetDataFromResolvedObject to update our recorded data
	UE_API virtual void RecordDataFromResolvedObject();

	// Applies our recorded data to the PropertyValuePtr for the resolved object
	UE_API virtual void ApplyDataToResolvedObject();

	// Returns the type of FProperty (FObjectProperty, FFloatProperty, etc)
	// If checking for Enums, prefer checking if GetEnumPropertyEnum() != nullptr, as it may be that
	// we received a FNumericProperty that actually represents an enum, which wouldn't be reflected here
	UE_API virtual FFieldClass* GetPropertyClass() const;
	UE_API EPropertyValueCategory GetPropCategory() const;
	UE_API virtual UScriptStruct* GetStructPropertyStruct() const;
	UE_API virtual UClass* GetObjectPropertyObjectClass() const;
	UE_API UEnum* GetEnumPropertyEnum() const;
	UE_API virtual bool ContainsProperty(const FProperty* Prop) const;

	// Returns an array of link segments that together describe the full property path
	UE_API const TArray<FCapturedPropSegment>& GetCapturedPropSegments() const;

	// Utility functions for UEnumProperties
	UE_API TArray<FName> GetValidEnumsFromPropertyOverride();
	UE_API FString GetEnumDocumentationLink();
	// Used RecordedData as an enum value and gets the corresponding index for our Enum
	UE_API int32 GetRecordedDataAsEnumIndex();
	// Sets our RecordedData with the value that matches Index, for our Enum
	UE_API void SetRecordedDataFromEnumIndex(int32 Index);
	// Makes sure RecordedData data is a valid Enum index for our Enum (_MAX is not allowed)
	UE_API void SanitizeRecordedEnumData();
	UE_API bool IsNumericPropertySigned();
	UE_API bool IsNumericPropertyUnsigned();
	UE_API bool IsNumericPropertyFloatingPoint();

	// Utility functions for string properties
	UE_API const FName& GetNamePropertyName() const;
	UE_API const FString& GetStrPropertyString() const;
	UE_API const FText& GetTextPropertyText() const;

	UE_API FName GetPropertyName() const;
	UFUNCTION(BlueprintCallable, Category="PropertyValue")
	UE_API FText GetPropertyTooltip() const;
	UFUNCTION(BlueprintCallable, Category="PropertyValue")
	UE_API const FString& GetFullDisplayString() const;
	UE_API FString GetLeafDisplayString() const;
	UE_API FString GetBaseDisplayString() const;
	UE_API virtual int32 GetValueSizeInBytes() const;
	UE_API int32 GetPropertyOffsetInBytes() const;

	UFUNCTION(BlueprintCallable, Category="PropertyValue")
	UE_API bool HasRecordedData() const;
	UE_API const TArray<uint8>& GetRecordedData();
	UE_API virtual void SetRecordedData(const uint8* NewDataBytes, int32 NumBytes, int32 Offset = 0);
	UE_API virtual const TArray<uint8>& GetDefaultValue();
	UE_API void ClearDefaultValue();
	// Returns true if our recorded data would remain the same if we called
	// RecordDataFromResolvedObject right now
	UE_API virtual bool IsRecordedDataCurrent();

	UE_API FOnPropertyApplied& GetOnPropertyApplied();
	UE_API FOnPropertyRecorded& GetOnPropertyRecorded();

#if WITH_EDITORONLY_DATA
	/**
	 * Get the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	UE_API uint32 GetDisplayOrder() const;

	/**
	 * Set the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	UE_API void SetDisplayOrder(uint32 InDisplayOrder);
#endif //WITH_EDITORONLY_DATA

protected:

	UE_API void SetRecordedDataInternal(const uint8* NewDataBytes, int32 NumBytes, int32 Offset = 0);

	UE_API FProperty* GetProperty() const;

	// Applies the recorded data to the TargetObject via the PropertySetter function
	// (e.g. SetIntensity instead of setting the Intensity UPROPERTY directly)
	UE_API virtual void ApplyViaFunctionSetter(UObject* TargetObject);

	// Recursively navigate the component/USCS_Node hierarchy trying to resolve our property path
	UE_API bool ResolveUSCSNodeRecursive(const USCS_Node* Node, int32 SegmentIndex);
	UE_API bool ResolvePropertiesRecursive(UStruct* ContainerClass, void* ContainerAddress, int32 PropertyIndex);

	FOnPropertyApplied OnPropertyApplied;
	FOnPropertyRecorded OnPropertyRecorded;
	
#if WITH_EDITOR
	UE_API void OnPIEEnded(const bool bIsSimulatingInEditor);
#endif

	// Temp data cached from last resolve
	FProperty* LeafProperty;
	UStruct* ParentContainerClass;
	void* ParentContainerAddress;
	UObject* ParentContainerObject; // Leafmost UObject* in the property path. Required as ParentContainerAddress
	uint8* PropertyValuePtr;        // may be pointing at a C++ struct
	UFunction* PropertySetter;

	// Properties were previously stored like this. Use CapturedPropSegments from now on, which stores
	// properties by name instead. It is much safer, as we can't guarantee these pointers will be valid
	// if they point at other packages (will depend on package load order, etc).
	UPROPERTY()
	TArray<TFieldPath<FProperty>> Properties_DEPRECATED;
	UPROPERTY()
	TArray<int32> PropertyIndices_DEPRECATED;

	UPROPERTY()
	TArray<FCapturedPropSegment> CapturedPropSegments;

	UPROPERTY()
	FString FullDisplayString;

	UPROPERTY()
	FName PropertySetterName;

	UPROPERTY()
	TMap<FString, FString> PropertySetterParameterDefaults;

	UPROPERTY()
	bool bHasRecordedData;

	// We use these mainly to know how to serialize/deserialize the values of properties that need special care
	// (e.g. UObjectProperties, name properties, text properties, etc)
	UPROPERTY()
	TObjectPtr<UClass> LeafPropertyClass_DEPRECATED;
	FFieldClass* LeafPropertyClass;

	UPROPERTY()
	TArray<uint8> ValueBytes;

	UPROPERTY()
	EPropertyValueCategory PropCategory;

	TArray<uint8> DefaultValue;

	TSoftObjectPtr<UObject> TempObjPtr;
	FName TempName;
	FString TempStr;
	FText TempText;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint32 DisplayOrder;
#endif //WITH_EDITORONLY_DATA
};

// Deprecated: Only here for backwards compatibility with 4.21
UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueTransform : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
};

// Deprecated: Only here for backwards compatibility
UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueVisibility : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
};

#undef UE_API
