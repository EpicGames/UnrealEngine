// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyValue.h"

#include "PropertyValueColor.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

// Keeps an FLinearColor interface by using the property setter/getter functions,
// even though the property itself is of FColor type
UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueColor : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

public:

	// UPropertyValue interface
	UE_API TArray<uint8> GetDataFromResolvedObject() const override;
	UE_API virtual UScriptStruct* GetStructPropertyStruct() const override;
	UE_API virtual int32 GetValueSizeInBytes() const override;
	//~ UPropertyValue interface
};

#undef UE_API
