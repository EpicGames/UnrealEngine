// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyValue.h"

#include "PropertyValueMaterial.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class UMaterialInterface;

UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueMaterial : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

public:

	UE_API UMaterialInterface* GetMaterial();
	UE_API void SetMaterial(UMaterialInterface* Mat);

	// Our leaf property will always be OverrideMaterials/OverrideMaterials[0] just for the
	// type/size/class information. It will normally fail to resolve if the StaticMeshComponent
	// is just using default materials though, so we have to intercept resolve calls and handle
	// them in a specific way. This will also let us zero out the value ptr and other things
	// that shouldn't be used by themselves
	UE_API virtual bool Resolve(UObject* OnObject = nullptr) override;

	UE_API virtual bool ContainsProperty(const FProperty* Prop) const override;

	UE_API virtual UStruct* GetPropertyParentContainerClass() const override;

	UE_API virtual TArray<uint8> GetDataFromResolvedObject() const override;
	UE_API virtual void ApplyDataToResolvedObject() override;

	UE_API virtual FFieldClass* GetPropertyClass() const override;
	UE_API virtual UClass* GetObjectPropertyObjectClass() const override;

	UE_API virtual int32 GetValueSizeInBytes() const override;

	UE_API virtual const TArray<uint8>& GetDefaultValue();

private:

	static FProperty* OverrideMaterialsProperty;
};

#undef UE_API
