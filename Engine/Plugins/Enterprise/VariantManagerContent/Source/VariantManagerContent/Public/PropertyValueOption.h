// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyValue.h"

#include "PropertyValueOption.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

// PropertyValue that can only be captured from ASwitchActors
UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueOption : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

public:
	// UPropertyValue interface
	UE_API virtual bool Resolve(UObject* OnObject = nullptr) override;
	UE_API virtual TArray<uint8> GetDataFromResolvedObject() const override;
	UE_API virtual void ApplyDataToResolvedObject() override;
	UE_API virtual const TArray<uint8>& GetDefaultValue() override;
	UE_API virtual int32 GetValueSizeInBytes() const override;
	//~ UPropertyValue interface
};

#undef UE_API
