// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyValue.h"

#include "PropertyValueSoftObject.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

/**
 * Stores data from a USoftObjectProperty.
 * It will store it's recorded data as a raw UObject*, and use the usual UPropertyValue
 * facilities for serializing it as a Soft object ptr. This derived class handles converting
 * to and from the property's underlying FSoftObjectPtr to our UObject*.
 * We can't keep a FSoftObjectPtr ourselves, neither as a temp member nor as raw bytes, as it has
 * internal heap-allocated data members like FName and FString.
 */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyValueSoftObject : public UPropertyValue
{
	GENERATED_UCLASS_BODY()

public:
	// UPropertyValue interface
	UE_API virtual int32 GetValueSizeInBytes() const override;
	UE_API virtual FFieldClass* GetPropertyClass() const override;

	UE_API virtual void ApplyDataToResolvedObject() override;
	UE_API virtual TArray<uint8> GetDataFromResolvedObject() const;

	UE_API virtual void ApplyViaFunctionSetter(UObject* TargetObject) override;

	UE_API virtual bool IsRecordedDataCurrent() override;

	UE_API virtual void SetRecordedData(const uint8* NewDataBytes, int32 NumBytes, int32 Offset = 0) override;
	//~ UPropertyValue interface
};

#undef UE_API
