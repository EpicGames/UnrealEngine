// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "SwitchActor.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

#define SWITCH_ACTOR_SELECTED_OPTION_NAME TEXT("Selected Option")

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSwitchActorSwitch, int32 /* new selected index */);

/**
 * Switch Actor allows quickly toggling the visibility of its child actors so that
 * only one is visible at a time. It can also be captured with the Variant Manager
 * to expose this capability as a property capture
 */
UCLASS(MinimalAPI)
class ASwitchActor : public AActor
{
public:

	GENERATED_BODY()

	UE_API ASwitchActor(const FObjectInitializer& Init);

	UFUNCTION(BlueprintCallable, CallInEditor, Category="SwitchActor", meta=(ToolTip="Returns the child actors that are available options, in a fixed order that may differ from the one displayed in the world outliner"))
	UE_API TArray<AActor*> GetOptions() const;

	UFUNCTION(BlueprintCallable, CallInEditor, Category="SwitchActor", meta=(ToolTip="If we have exactly one child actor visible, it will return a pointer to it. Returns nullptr otherwise"))
	UE_API int32 GetSelectedOption() const;

	UFUNCTION(BlueprintCallable, CallInEditor, Category="SwitchActor", meta=(ToolTip="Select one of the available options by index"))
	UE_API void SelectOption(int32 OptionIndex);

	UE_API FOnSwitchActorSwitch& GetOnSwitchDelegate();

private:
	// Dedicated function to set our visibility so that we can restore our component
	// hierarchy to the last-set option, in case e.g. we're overriden by a parent ASwitchActor
	void SetVisibility(bool bVisible);

	UE_API void PostLoad() override;

	FOnSwitchActorSwitch OnSwitchActorSwitch;

	// Exposing our root component like this allows manual Mobility control on the details panel
	UPROPERTY(Category = SwitchActor, VisibleAnywhere)
	TObjectPtr<class USceneComponent> SceneComponent;

	UPROPERTY()
	int32 LastSelectedOption;
};

#undef UE_API
