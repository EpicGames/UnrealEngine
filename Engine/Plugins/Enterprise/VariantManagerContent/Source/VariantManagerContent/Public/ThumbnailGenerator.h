// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"

#define VARIANT_MANAGER_THUMBNAIL_SIZE 256

class UTexture2D;

namespace ThumbnailGenerator
{
    UTexture2D* GenerateThumbnailFromTexture(UTexture2D* InImage);
    UTexture2D* GenerateThumbnailFromFile(FString FilePath);
    UTexture2D* GenerateThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees = 50.0f, float MinZ = 50.0f, float Gamma = 2.2f);
    UTexture2D* GenerateThumbnailFromEditorViewport();
	UTexture2D* GenerateThumbnailFromObjectThumbnail(UObject* Object);
}
