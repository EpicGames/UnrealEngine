// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Variant.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class ULevelVariantSets;
class UTexture2D;

class UVariant;
class UVariantSet;
class UVariantObjectBinding;
struct FVariantImpl;

USTRUCT( BlueprintType )
struct FVariantDependency
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Dependency")
	TSoftObjectPtr<UVariantSet> VariantSet;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Dependency")
	TSoftObjectPtr<UVariant> Variant;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Dependency")
	bool bEnabled = true;
};

UCLASS(MinimalAPI, BlueprintType)
class UVariant : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	friend FVariantImpl;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVariantChanged, UVariant*);

	// Broadcast whenever we modify a variant thumbnail
	static UE_API FOnVariantChanged OnThumbnailUpdated;

	// Broadcast whenever we add/remove/modify a variant dependency
	static UE_API FOnVariantChanged OnDependenciesUpdated;

public:
	UFUNCTION(BlueprintPure, Category="Variant")
	UE_API class UVariantSet* GetParent();

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	UFUNCTION(BlueprintCallable, Category="Variant")
	UE_API void SetDisplayText(const FText& NewDisplayText);

	UFUNCTION(BlueprintPure, Category="Variant")
	UE_API FText GetDisplayText() const;

	// In case of a duplicate binding these will destroy the older bindings
	UE_API void AddBindings(const TArray<UVariantObjectBinding*>& NewBindings, int32 Index = INDEX_NONE);
	UE_API int32 GetBindingIndex(UVariantObjectBinding* Binding);
	UE_API const TArray<UVariantObjectBinding*>& GetBindings() const;
	UE_API void RemoveBindings(const TArray<UVariantObjectBinding*>& Bindings);

	UFUNCTION(BlueprintPure, Category="Variant")
	UE_API int32 GetNumActors();

	UFUNCTION(BlueprintPure, Category="Variant")
	UE_API AActor* GetActor(int32 ActorIndex);

	UE_API UVariantObjectBinding* GetBindingByName(const FString& ActorName);

	UFUNCTION(BlueprintCallable, Category="Variant")
	UE_API void SwitchOn();

	// Returns true if none of our properties are dirty
	UFUNCTION(BlueprintCallable, Category="Variant")
	UE_API bool IsActive();

	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail")
	UE_API void SetThumbnailFromObject(UObject* ObjectThumbnail);

	// Sets the thumbnail to use for this variant. Can receive nullptr to clear it
	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail")
	UE_API void SetThumbnailFromTexture(UTexture2D* NewThumbnail);

	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail")
	UE_API void SetThumbnailFromFile(FString FilePath);

	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail", meta=(WorldContext = "WorldContextObject"))
	UE_API void SetThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees = 50.0f, float MinZ = 50.0f, float Gamma = 2.2f);

	// Sets the thumbnail from the active editor viewport. Doesn't do anything if the Editor is not available
	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail", meta=(CallInEditor="true"))
	UE_API void SetThumbnailFromEditorViewport();

	// Gets the thumbnail currently used for this variant
	UFUNCTION(BlueprintCallable, Category="Variant|Thumbnail")
	UE_API UTexture2D* GetThumbnail();

	// Returns all the variants that have this variant as a dependency
	UFUNCTION(BlueprintCallable, Category="Variant|Dependencies")
	UE_API TArray<UVariant*> GetDependents(ULevelVariantSets* LevelVariantSets, bool bOnlyEnabledDependencies);

	// Returns if we can safely trigger Other as a dependency without the danger of cycles
	UE_API bool IsValidDependency(const UVariant* Other) const;

	UFUNCTION(Category = "Variant|Dependencies")
	UE_API int32 AddDependency(FVariantDependency& Dependency);

	// Returning by reference in blueprint doesn't seem to work if we want to later modify the FVariantDependency, so
	// here we leave the return type without UPARAM(ref) so C++ code can use it as ref, while this function in blueprint returns by value
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Variant|Dependencies", meta=(ToolTip="Get the dependency at index 'Index' by value. Will crash if index is invalid"))
	UE_API FVariantDependency& GetDependency(int32 Index);

	UFUNCTION(Category = "Variant|Dependencies")
	UE_API void SetDependency(int32 Index, FVariantDependency& Dependency);

	UFUNCTION(Category = "Variant|Dependencies")
	UE_API void DeleteDependency(int32 Index);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Variant|Dependencies")
	UE_API int32 GetNumDependencies();

private:
	UE_API void SetThumbnailInternal(UTexture2D* NewThumbnail);

private:
	UPROPERTY()
	TArray<FVariantDependency> Dependencies;

	// The display name used to be a property. Use the non-deprecated, non-property version from now on
	UPROPERTY()
	FText DisplayText_DEPRECATED;

	FText DisplayText;

	UPROPERTY()
	TArray<TObjectPtr<UVariantObjectBinding>> ObjectBindings;

	UPROPERTY()
	TObjectPtr<UTexture2D> Thumbnail;

#if WITH_EDITOR
	// Whether we already tried restoring a thumbnail from the actual package (backwards compatibility)
	bool bTriedRestoringOldThumbnail;
#endif
};

#undef UE_API
