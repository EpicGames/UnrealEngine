// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()

#define VARIANTMANAGERCONTENTMODULE_MODULE_NAME TEXT("VariantManagerContent")


class IVariantManagerContentModule : public IModuleInterface
{
public:
	static inline IVariantManagerContentModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IVariantManagerContentModule>(VARIANTMANAGERCONTENTMODULE_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(VARIANTMANAGERCONTENTMODULE_MODULE_NAME);
	}
};

