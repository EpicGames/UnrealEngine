// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "VariantObjectBinding.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

struct FFunctionCaller;

class UPropertyValue;

UCLASS(MinimalAPI, DefaultToInstanced, meta=(ScriptName="UVariantActorBinding"))
class UVariantObjectBinding : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UE_API void SetObject(UObject* InObject);

	UE_API class UVariant* GetParent();

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	UE_API FText GetDisplayText() const;

	UE_API FString GetObjectPath() const;
	UE_API UObject* GetObject() const;

	UE_API void AddCapturedProperties(const TArray<UPropertyValue*>& Properties);
	UE_API const TArray<UPropertyValue*>& GetCapturedProperties() const;
	UE_API void RemoveCapturedProperties(const TArray<UPropertyValue*>& Properties);
	UE_API void SortCapturedProperties();

	UE_API void AddFunctionCallers(const TArray<FFunctionCaller>& InFunctionCallers);
	UE_API TArray<FFunctionCaller>& GetFunctionCallers();
	UE_API const TArray<FFunctionCaller>& GetFunctionCallers() const;
	UE_API void RemoveFunctionCallers(const TArray<FFunctionCaller*>& InFunctionCallers);
	UE_API void ExecuteTargetFunction(FName FunctionName);
	UE_API void ExecuteAllTargetFunctions();

#if WITH_EDITORONLY_DATA
	UE_API void UpdateFunctionCallerNames();
#endif

private:
	/**
	 * Whenever we resolve, we cache the actor label here so that if we can't
	 * resolve anymore we can better indicate which actor is missing, instead of just
	 * saying 'Unloaded binding'
	 */
	UPROPERTY()
	mutable FString CachedActorLabel;

	UPROPERTY()
	mutable FSoftObjectPath ObjectPtr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	mutable TLazyObjectPtr<UObject> LazyObjectPtr_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<TObjectPtr<UPropertyValue>> CapturedProperties;

	UPROPERTY()
	TArray<FFunctionCaller> FunctionCallers;
};

#undef UE_API
