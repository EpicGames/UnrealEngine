// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "VariantSet.generated.h"

#define UE_API VARIANTMANAGERCONTENT_API

class UTexture2D;

class UVariant;

UCLASS(MinimalAPI, BlueprintType)
class UVariantSet : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVariantSetChanged, UVariantSet*);
	static UE_API FOnVariantSetChanged OnThumbnailUpdated;

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API class ULevelVariantSets* GetParent();

	// UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	// Sets whether this variant set is expanded or not when displayed
	// in a variant manager
	UE_API bool IsExpanded() const;
	UE_API void SetExpanded(bool bInExpanded);

	UFUNCTION(BlueprintCallable, Category="VariantSet")
	UE_API void SetDisplayText(const FText& NewDisplayText);

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API FText GetDisplayText() const;

	UE_API FString GetUniqueVariantName(const FString& InPrefix) const;

	UE_API void AddVariants(const TArray<UVariant*>& NewVariants, int32 Index = INDEX_NONE);
	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API int32 GetVariantIndex(UVariant* Var) const;
	UE_API const TArray<UVariant*>& GetVariants() const;
	UE_API void RemoveVariants(const TArray<UVariant*>& InVariants);

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API int32 GetNumVariants() const;

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API UVariant* GetVariant(int32 VariantIndex);

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API UVariant* GetVariantByName(FString VariantName);

	UFUNCTION(BlueprintPure, Category="VariantSet")
	UE_API int32 GetVariantIndexByName(FString VariantName) const;

	// Sets the thumbnail to use for this variant set. Can receive nullptr to clear it
	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail")
	UE_API void SetThumbnailFromObject(UObject* ObjectThumbnail);

	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail")
	UE_API void SetThumbnailFromTexture(UTexture2D* NewThumbnail);

	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail")
	UE_API void SetThumbnailFromFile(FString FilePath);

	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail", meta = (WorldContext = "WorldContextObject"))
	UE_API void SetThumbnailFromCamera(UObject* WorldContextObject, const FTransform& CameraTransform, float FOVDegrees = 50.0f, float MinZ = 50.0f, float Gamma = 2.2f);

	// Sets the thumbnail from the active editor viewport. Doesn't do anything if the Editor is not available
	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail", meta = (CallInEditor = "true"))
	UE_API void SetThumbnailFromEditorViewport();

	// Gets the thumbnail currently used for this variant set
	UFUNCTION(BlueprintCallable, Category = "VariantSet|Thumbnail")
	UE_API UTexture2D* GetThumbnail();

private:
	void SetThumbnailInternal(UTexture2D* NewThumbnail);

private:

	// The display name used to be a property. Use the non-deprecated, non-property version from now on
	UPROPERTY()
	FText DisplayText_DEPRECATED;

	FText DisplayText;

	UPROPERTY()
	bool bExpanded;

	UPROPERTY()
	TArray<TObjectPtr<UVariant>> Variants;

	UPROPERTY()
	TObjectPtr<UTexture2D> Thumbnail;
};

#undef UE_API
