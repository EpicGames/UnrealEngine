// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class VariantManagerContent : ModuleRules
	{
		public VariantManagerContent(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RenderCore",
					"RHI",
				}
			);

			// For managing direct function entry nodes for FunctionCallers
			// Necessary when in the editor as the functions may get recompiled/renamed/deleted
			// and tracking them just by name would not work
			// Also for converting old thumbnails to new format
			if (Target.bBuildWithEditorOnlyData)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"EditorFramework",
						"UnrealEd",
						"BlueprintGraph"
					}
				);
			}
		}
	}
}