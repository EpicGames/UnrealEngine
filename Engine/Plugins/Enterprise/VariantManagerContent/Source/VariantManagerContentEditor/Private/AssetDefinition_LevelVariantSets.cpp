// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_LevelVariantSets.h"

#include "ContentBrowserMenuContexts.h"
#include "LevelVariantSets.h"
#include "ToolMenuSection.h"
#include "VariantManagerContentEditorModule.h"
#include "ToolMenus.h"
#include "VariantManagerContentStyle.h"
#include "Styling/SlateStyleRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_LevelVariantSets)

#define LOCTEXT_NAMESPACE "LevelVariantSetAssetActions"

FText UAssetDefinition_LevelVariantSets::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_LevelVariantSets", "Level Variant Sets");
}

TSoftClassPtr<> UAssetDefinition_LevelVariantSets::GetAssetClass() const
{
	return ULevelVariantSets::StaticClass();
}

EAssetCommandResult UAssetDefinition_LevelVariantSets::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (ULevelVariantSets* LevelVariantSets : OpenArgs.LoadObjects<ULevelVariantSets>())
	{
		IVariantManagerContentEditorModule& ContentEditorModule = IVariantManagerContentEditorModule::Get();
		ContentEditorModule.GetOnLevelVariantSetsEditorOpened().ExecuteIfBound(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, LevelVariantSets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_LevelVariantSets::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Data, LOCTEXT("LevelVariantSets", "Other"), ECategoryMenuType::Section)
		};
	return Categories;
}

FAssetSupportResponse UAssetDefinition_LevelVariantSets::CanLocalize(const FAssetData& InAsset) const
{
	return FAssetSupportResponse::NotSupported();
}

FAssetOpenSupport UAssetDefinition_LevelVariantSets::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit, EToolkitMode::WorldCentric); 
}

const FSlateBrush* UAssetDefinition_LevelVariantSets::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FVariantManagerContentStyle::Get().GetBrush("VariantManager.Icon");
}

namespace MenuExtension_LevelVariantSets
{
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(ULevelVariantSets::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					TArray<FAssetData> Assets = CBContext->GetSelectedAssetsOfType(ULevelVariantSets::StaticClass());

					InSection.AddMenuEntry(
						"LevelVariantSetAssetActions_CreateActorText",
						LOCTEXT("CreateActorText", "Create LevelVariantSets actor"),
						LOCTEXT("CreateActorTooltip", "Creates a new ALevelVariantSetsActor AActor and add it to the scene"),
						FSlateIcon(),
						FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
						{
							if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(Context))
							{
								IVariantManagerContentEditorModule& ContentEditorModule = IVariantManagerContentEditorModule::Get();
								for (ULevelVariantSets* LevelVariantSets : CBContext->LoadSelectedObjects<ULevelVariantSets>())
								{
									ContentEditorModule.GetOrCreateLevelVariantSetsActor(LevelVariantSets, true);
								}
							}
						}));
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
