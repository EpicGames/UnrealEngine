// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetTypeActions_Base.h"

#include "AssetDefinition_LevelVariantSets.generated.h"


UCLASS()
class UAssetDefinition_LevelVariantSets : public UAssetDefinitionDefault
{
private:
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FColor(80, 80, 200); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return false; }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	// UAssetDefinition End
};
