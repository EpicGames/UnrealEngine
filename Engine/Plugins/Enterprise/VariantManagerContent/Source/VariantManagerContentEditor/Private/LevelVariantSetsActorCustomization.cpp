// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSetsActorCustomization.h"

#include "LevelVariantSets.h"
#include "LevelVariantSetsActor.h"
#include "VariantManagerContentEditorLog.h"
#include "VariantManagerContentEditorModule.h"

#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LevelVariantSetsActorCustomization"

FLevelVariantSetsActorCustomization::FLevelVariantSetsActorCustomization()
{
}

TSharedRef<IDetailCustomization> FLevelVariantSetsActorCustomization::MakeInstance()
{
	return MakeShared<FLevelVariantSetsActorCustomization>();
}

void FLevelVariantSetsActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	//Default to standard details panel when multiple variantsselectors are selected at once
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
	if (!SelectedObject.IsValid())
	{
		return;
	}

	ALevelVariantSetsActor* Actor = Cast<ALevelVariantSetsActor>(SelectedObject.Get());

	IDetailCategoryBuilder& ActionsCategory = DetailLayoutBuilder.EditCategory(TEXT("VariantManager"));

	ActionsCategory.AddCustomRow(FText())
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Text(LOCTEXT("OpenVariantManager", "Open Variant Manager"))
			.OnClicked(this, &FLevelVariantSetsActorCustomization::OnOpenVariantManagerButtonClicked, Actor)
		]
	];

	ActionsCategory.AddCustomRow(FText())
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Text(LOCTEXT("NewLevelVarSets", "Create new LevelVariantSets asset"))
			.OnClicked(this, &FLevelVariantSetsActorCustomization::OnCreateLevelVarSetsButtonClicked)
		]
	];

	ActionsCategory.AddProperty(DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ALevelVariantSetsActor, LevelVariantSets)));
}

FReply FLevelVariantSetsActorCustomization::OnOpenVariantManagerButtonClicked(ALevelVariantSetsActor* Actor)
{
	if (Actor == nullptr || !Actor->IsValidLowLevel())
	{
		return FReply::Unhandled();
	}

	ULevelVariantSets* LevelVarSets = Actor->GetLevelVariantSets(true);
	if (LevelVarSets == nullptr || !LevelVarSets->IsValidLowLevel())
	{
		return FReply::Unhandled();
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets({LevelVarSets});

	return FReply::Handled();
}

FReply FLevelVariantSetsActorCustomization::OnCreateLevelVarSetsButtonClicked()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	IVariantManagerContentEditorModule& ContentEditorModule = IVariantManagerContentEditorModule::Get();

	UObject* NewAsset = ContentEditorModule.CreateLevelVariantSetsAssetWithDialog();
	if (!NewAsset)
	{
		UE_LOGF(LogVariantManagerContentEditor, Error, "Did not find an asset factory for a ULevelVariantSets");
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
