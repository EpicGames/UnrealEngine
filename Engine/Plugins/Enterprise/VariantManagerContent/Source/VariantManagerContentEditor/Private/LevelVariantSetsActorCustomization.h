// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class ALevelVariantSetsActor;
class FReply;
class IDetailLayoutBuilder;

class FLevelVariantSetsActorCustomization : public IDetailCustomization
{
public:
	FLevelVariantSetsActorCustomization();
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	FReply OnOpenVariantManagerButtonClicked(ALevelVariantSetsActor* Actor);
	FReply OnCreateLevelVarSetsButtonClicked();
};
