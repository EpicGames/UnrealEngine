// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSetsActorFactory.h"

#include "LevelVariantSets.h"
#include "LevelVariantSetsActor.h"

#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelVariantSetsActorFactory)

#define LOCTEXT_NAMESPACE "ALevelVariantSetsActorFactory"

ULevelVariantSetsActorFactory::ULevelVariantSetsActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ALevelVariantSetsActorDisplayName", "LevelVariantSetsActor");
	NewActorClass = ALevelVariantSetsActor::StaticClass();
}

bool ULevelVariantSetsActorFactory::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf( ULevelVariantSets::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoLevelVariantSetsAsset", "A valid variant sets asset must be specified.");
		return false;
	}

	return true;
}

AActor* ULevelVariantSetsActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	ALevelVariantSetsActor* NewActor = Cast<ALevelVariantSetsActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));

	if (NewActor)
	{
		if (ULevelVariantSets* LevelVariantSets = Cast<ULevelVariantSets>(InAsset))
		{
			NewActor->SetLevelVariantSets(LevelVariantSets);
		}
	}

	return NewActor;
}

UObject* ULevelVariantSetsActorFactory::GetAssetFromActorInstance(AActor* Instance)
{
	if (ALevelVariantSetsActor* LevelVariantSetsActor = Cast<ALevelVariantSetsActor>(Instance))
	{
		return LevelVariantSetsActor->LevelVariantSets.TryLoad();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
