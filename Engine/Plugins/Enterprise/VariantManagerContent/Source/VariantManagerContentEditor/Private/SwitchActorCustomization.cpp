// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchActorCustomization.h"

#include "Framework/Views/TableViewMetadata.h"
#include "SwitchActor.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SwitchActorCustomization"

FSwitchActorCustomization::FSwitchActorCustomization()
{
}

TSharedRef<IDetailCustomization> FSwitchActorCustomization::MakeInstance()
{
	return MakeShared<FSwitchActorCustomization>();
}

void FSwitchActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	//Default to standard details panel when multiple switch actors are selected at once
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
	if (!SelectedObject.IsValid())
	{
		return;
	}

	CurrentActor = Cast<ASwitchActor>(SelectedObject.Get());
	CurrentActor->GetOnSwitchDelegate().AddSP(this, &FSwitchActorCustomization::ForceRefreshDetails);

	TArray<AActor*> ActorOptions = CurrentActor->GetOptions();

	Options.Reset();
	Algo::Transform(ActorOptions, Options, [](const AActor* Actor)
	{
		return MakeShared<FString>(Actor->GetActorLabel());
	});

	TSharedPtr<FString>* InitiallySelected = nullptr;
	int32 SelectedIndex = CurrentActor->GetSelectedOption();
	if (Options.IsValidIndex(SelectedIndex))
	{
		InitiallySelected = &Options[SelectedIndex];
	}

	IDetailCategoryBuilder& ActionsCategory = DetailLayoutBuilder.EditCategory(TEXT("SwitchActor"));
	const FText FilterStr = FText::FromString( SWITCH_ACTOR_SELECTED_OPTION_NAME );
	ActionsCategory.AddCustomRow(FilterStr)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FilterStr)
		.ToolTipText(LOCTEXT("SelectedOptionTooltip", "Which child actor is visible"))
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SAssignNew(ComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&Options)
		.InitiallySelectedItem(InitiallySelected? *InitiallySelected : nullptr)
		.OnComboBoxOpening(this, &FSwitchActorCustomization::OnComboBoxOpening)
		.OnSelectionChanged(this, &FSwitchActorCustomization::OnComboBoxOptionChanged)
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
		{
			return SNew(STextBlock).Text(Item.IsValid() ? FText::FromString(*Item) : FText::GetEmpty());
		})
		.Content()
		[
			SNew(STextBlock).Text(this, &FSwitchActorCustomization::GetComboBoxSelectedOptionText)
		]
	];
}

void FSwitchActorCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetailBuilderWeakPtr = DetailBuilder;
	CustomizeDetails( *DetailBuilder );
}

void FSwitchActorCustomization::OnComboBoxOpening()
{
	if (CurrentActor == nullptr)
	{
		return;
	}

	TArray<AActor*> Actors = CurrentActor->GetOptions();

	Options.Reset();
	Algo::Transform(Actors, Options, [](const AActor* Actor)
	{
		return MakeShared<FString>(Actor->GetActorLabel());
	});

	ComboBox->ClearSelection();
	ComboBox->RefreshOptions();

	// Need this to prevent it from clearing the selected option,
	// as we'll need to rebuild the Options array to make sure its
	// up to date
	int32 SelectedOption = CurrentActor->GetSelectedOption();
	if (Options.IsValidIndex(SelectedOption))
	{
		ComboBox->SetSelectedItem(Options[SelectedOption]);
	}
}

void FSwitchActorCustomization::OnComboBoxOptionChanged(TSharedPtr<FString> NewOption, ESelectInfo::Type SelectType)
{
	// Checking SelectType prevents a refresh loop when setting the default selection in OnComboBoxOpening
	if (CurrentActor == nullptr || !NewOption.IsValid() || SelectType == ESelectInfo::Direct)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("OptionChangedTransaction", "Switched '{0}' to option '{1}'"),
		FText::FromString(CurrentActor->GetActorLabel()),
		FText::FromString(NewOption.IsValid()? *NewOption : TEXT("None")))
	);

	// NewOption is newly generated, so it won't match by address of the pointers stored in Options
	int32 SelectedIndex = INDEX_NONE;
	for (int32 Index = 0; Index < Options.Num(); ++Index)
	{
		const TSharedPtr<FString>& Option = Options[Index];
		if (Option.IsValid() && *Option == *NewOption)
		{
			SelectedIndex = Index;
			break;
		}
	}

	CurrentActor->SelectOption(SelectedIndex);

	// Trick to force the viewport to update even with 'Realtime' turned off,
	// even though our selection may will remain the same
	GEditor->NoteSelectionChange();
}

FText FSwitchActorCustomization::GetComboBoxSelectedOptionText() const
{
	TSharedPtr<FString> SelectedItem = ComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

void FSwitchActorCustomization::ForceRefreshDetails(int32 NewOption)
{
	// Raw because we don't want to keep alive the details builder when calling the force refresh details
	IDetailLayoutBuilder* DetailLayoutBuilder = DetailBuilderWeakPtr.Pin().Get();
	if ( DetailLayoutBuilder )
	{
		DetailLayoutBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
