// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/Input/SComboBox.h"

class ASwitchActor;
class IDetailLayoutBuilder;

class FSwitchActorCustomization : public IDetailCustomization
{
public:
	FSwitchActorCustomization();
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	// End of IDetailCustomization interface

private:
	void OnComboBoxOpening();
	void OnComboBoxOptionChanged(TSharedPtr<FString> NewOption, ESelectInfo::Type SelectType);
	FText GetComboBoxSelectedOptionText() const;
	void ForceRefreshDetails(int32 NewOption);

	ASwitchActor* CurrentActor;
	TWeakPtr<IDetailLayoutBuilder> DetailBuilderWeakPtr;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBox;
	TArray<TSharedPtr<FString>> Options;
};
