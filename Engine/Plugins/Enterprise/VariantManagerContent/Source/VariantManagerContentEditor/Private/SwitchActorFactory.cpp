// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchActorFactory.h"
#include "SwitchActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SwitchActorFactory)

USwitchActorFactory::USwitchActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = NSLOCTEXT("SwitchActorFactory", "SwitchActorDisplayName", "Switch Actor");
	NewActorClass = ASwitchActor::StaticClass();
}
