// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerContentEditorModule.h"

#include "Editor.h"
#include "LevelVariantSets.h"
#include "LevelVariantSetsActor.h"
#include "LevelVariantSetsActorCustomization.h"
#include "SwitchActorCustomization.h"
#include "UObject/UObjectIterator.h"
#include "VariantManagerContentEditorLog.h"

#include "AssetToolsModule.h"
#include "Factories/Factory.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "PropertyEditorModule.h"
#include "VariantManagerContentStyle.h"

#define LOCTEXT_NAMESPACE "VariantManagerContentEditorModule"

DEFINE_LOG_CATEGORY(LogVariantManagerContentEditor);

class FVariantManagerContentEditorModule : public IVariantManagerContentEditorModule
{
public:
	virtual void StartupModule() override
	{
		FVariantManagerContentStyle::Initialize();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(TEXT("LevelVariantSetsActor"), FOnGetDetailCustomizationInstance::CreateStatic(&FLevelVariantSetsActorCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(TEXT("SwitchActor"), FOnGetDetailCustomizationInstance::CreateStatic(&FSwitchActorCustomization::MakeInstance));
	}

	virtual void ShutdownModule() override
	{
        if (FPropertyEditorModule* PropertyModule = FModuleManager::LoadModulePtr< FPropertyEditorModule >(TEXT("PropertyEditor")))
        {
        	PropertyModule->UnregisterCustomClassLayout(TEXT("LevelVariantSetsActor"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("SwitchActor"));
		}

		FVariantManagerContentStyle::Shutdown();
	}

	virtual void RegisterOnLevelVariantSetsDelegate(FOnLevelVariantSetsEditor Delegate) override
	{
		OnLevelVariantSetsEditor = Delegate;
	}

	virtual void UnregisterOnLevelVariantSetsDelegate() override
	{
		OnLevelVariantSetsEditor.Unbind();
	}

	virtual FOnLevelVariantSetsEditor GetOnLevelVariantSetsEditorOpened() const override
	{
		return OnLevelVariantSetsEditor;
	}

    virtual UObject* CreateLevelVariantSetsAsset(const FString& AssetName, const FString& PackagePath, bool bForceOverwrite) override
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		UObject* NewAsset = nullptr;

		FString SafePackagePath = UPackageTools::SanitizePackageName(PackagePath);
		FString SafeAssetName = ObjectTools::SanitizeObjectName(AssetName);

		if (SafePackagePath.IsEmpty() || SafeAssetName.IsEmpty())
		{
			UE_LOGF(LogVariantManagerContentEditor, Error, "Invalid ULevelVariantSets asset name or package path!");
			return nullptr;
		}

		// Attempt to create a new asset by scanning all available factories until we find one that supports ULevelVariantSets
		for (TObjectIterator<UClass> It ; It ; ++It)
		{
			UClass* CurrentClass = *It;
			if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
			{
				UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
				if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == ULevelVariantSets::StaticClass())
				{
					// Try deleting existing asset
					// CreateAsset will always call CanCreateAsset, which will spawn a dialog in case of conflicts.
					// We want to avoid the dialog, but also still use CreateAsset so that only VariantManagerFactoryNew is responsible for
					// creating LVS assets. This means we have to manually (try to) delete the asset first
					// Reference: UAssetToolsImpl::CanCreateAsset
					if (bForceOverwrite)
					{
						const FString PackageName = SafePackagePath + TEXT("/") + SafeAssetName;
						UPackage* Pkg = CreatePackage(*PackageName);

						// Search for UObject instead of ULevelVariantSets as we also want to catch redirectors
						UObject* ExistingObject = StaticFindObject( UObject::StaticClass(), Pkg, *SafeAssetName );

						if (ExistingObject != nullptr)
						{
							// Try to fixup a redirector before we delete it
							if (ExistingObject->GetClass()->IsChildOf(UObjectRedirector::StaticClass()))
							{
								AssetTools.FixupReferencers({Cast<UObjectRedirector>(ExistingObject)});
							}

							bool bDeleteSucceeded = ObjectTools::DeleteSingleObject( ExistingObject );
							if (bDeleteSucceeded)
							{
								// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
								CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

								// Old package will be GC'ed... create a new one here
								Pkg = CreatePackage(*PackageName);
								Pkg->MarkAsFullyLoaded();
							}

							if (!bDeleteSucceeded || !IsUniqueObjectName(*AssetName, Pkg))
							{
								UE_LOGF(LogVariantManagerContentEditor, Error, "Failed to delete old ULevelVariantSets asset at '%ls'. New asset can't be created.", *PackageName);
								return nullptr;
							}
						}
					}

					NewAsset = AssetTools.CreateAsset(SafeAssetName, SafePackagePath, ULevelVariantSets::StaticClass(), Factory);
					break;
				}
			}
		}

		return NewAsset;
	}

	virtual UObject* CreateLevelVariantSetsAssetWithDialog() override
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		UObject* NewAsset = nullptr;

		// Attempt to create a new asset by scanning all available factories until we find one that supports ULevelVariantSets
		for (TObjectIterator<UClass> It ; It ; ++It)
		{
			UClass* CurrentClass = *It;
			if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
			{
				UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
				if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == ULevelVariantSets::StaticClass())
				{
					NewAsset = AssetTools.CreateAssetWithDialog(ULevelVariantSets::StaticClass(), Factory);
					break;
				}
			}
		}

		return NewAsset;
	}

	virtual AActor* GetOrCreateLevelVariantSetsActor(UObject* LevelVariantSetsAsset, bool bForceCreate) override
	{
		if (!bForceCreate)
		{
			// Check to see if we have a LVSA for this asset already
			TArray< UObject* > WorldObjects;
			GetObjectsWithOuter( GWorld, WorldObjects );

			for( UObject* Object : WorldObjects )
			{
				if ( ALevelVariantSetsActor* Actor = Cast< ALevelVariantSetsActor >( Object ) )
				{
					if ( Actor->GetLevelVariantSets(true) == LevelVariantSetsAsset && !Actor->IsActorBeingDestroyed() )
					{
						return Actor;
					}
				}
			}
		}

		UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ALevelVariantSetsActor::StaticClass());
		if (ActorFactory == nullptr)
		{
			UE_LOGF(LogVariantManagerContentEditor, Error, "Failed to create LevelVariantSetsActor!");
			return nullptr;
		}

		return GEditor->UseActorFactory(ActorFactory, FAssetData(LevelVariantSetsAsset), &FTransform::Identity);
	}

private:
	FOnLevelVariantSetsEditor OnLevelVariantSetsEditor;
};

IMPLEMENT_MODULE(FVariantManagerContentEditorModule, VariantManagerContentEditor);

#undef LOCTEXT_NAMESPACE

