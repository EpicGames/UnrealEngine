// Copyright Epic Games, Inc. All Rights Reserved.


#include "VariantManagerContentStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"


FVariantManagerContentStyle::FVariantManagerContentStyle() : FSlateStyleSet("VariantManagerContentStyle")
{
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("VariantManagerContent")))
	{
		SetContentRoot(Plugin->GetContentDir());
	}

	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	Set( "VariantManager.Icon", new IMAGE_BRUSH_SVG("VariantManager_64", Icon64x64) );

	FSlateStyleRegistry::RegisterSlateStyle(*this);

}

void FVariantManagerContentStyle::Initialize()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());

	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FVariantManagerContentStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

const FName FVariantManagerContentStyle::GetAppStyleSetName()
{
	return FName("VariantManagerContentStyle");
}
