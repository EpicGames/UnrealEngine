// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

struct FVariantManagerContentStyle : public FSlateStyleSet
{
public:
	FVariantManagerContentStyle();

	static void Initialize();

	static void Shutdown();

	static const FName GetAppStyleSetName();

	static FVariantManagerContentStyle& Get()
	{
		static FVariantManagerContentStyle Inst;
		return Inst;
	}
};