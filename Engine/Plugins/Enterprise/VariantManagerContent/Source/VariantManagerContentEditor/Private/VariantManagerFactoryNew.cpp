// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerFactoryNew.h"

#include "LevelVariantSets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantManagerFactoryNew)

UVariantManagerFactoryNew::UVariantManagerFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = ULevelVariantSets::StaticClass();
}

UObject* UVariantManagerFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	ULevelVariantSets* VariantSets = NewObject<ULevelVariantSets>(InParent, Name, Flags|RF_Transactional);

	return VariantSets;
}

bool UVariantManagerFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
