// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerTestActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantManagerTestActor)



AVariantManagerTestActor::AVariantManagerTestActor(const FObjectInitializer& Init)
	: Super(Init)
{
}
