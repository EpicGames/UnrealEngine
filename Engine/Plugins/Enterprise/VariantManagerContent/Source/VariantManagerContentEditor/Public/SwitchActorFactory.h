// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "SwitchActorFactory.generated.h"

// The only purpose of this class is to trigger a slightly different code path within
// FLevelEditorViewportClient::TryPlacingActorFromObject when dragging and dropping a SwitchActor into the
// viewport, so that the SwitchActor labels get sanitized by FActorLabelUtilities::SetActorLabelUnique
// and don't repeatedly increment
UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class USwitchActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()
};
