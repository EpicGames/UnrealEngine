// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()
#include "Toolkits/IToolkit.h"

#define VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME TEXT("VariantManagerContentEditor")

class AActor;
class ULevelVariantSets;

DECLARE_DELEGATE_ThreeParams( FOnLevelVariantSetsEditor, const EToolkitMode::Type, const TSharedPtr< class IToolkitHost >&, class ULevelVariantSets*);

class IVariantManagerContentEditorModule : public IModuleInterface
{
public:
	static inline IVariantManagerContentEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IVariantManagerContentEditorModule>(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
	}

	// This will be executed whenever FLevelVariantSetsAssetActions::OpenAssetEditor is called with a valid asset.
	// This so that the editor toolkit can be on the VariantManager plugin and the AssetActions can be on this module
	virtual void RegisterOnLevelVariantSetsDelegate(FOnLevelVariantSetsEditor Delegate) = 0;
	virtual void UnregisterOnLevelVariantSetsDelegate() = 0;
	virtual FOnLevelVariantSetsEditor GetOnLevelVariantSetsEditorOpened() const = 0;

	virtual UObject* CreateLevelVariantSetsAssetWithDialog() = 0;
	virtual UObject* CreateLevelVariantSetsAsset(const FString& AssetName, const FString& PackagePath, bool bForceOverwrite = false) = 0;
	virtual AActor* GetOrCreateLevelVariantSetsActor(UObject* LevelVariantSetsAsset, bool bForceCreate=false) = 0;
};


