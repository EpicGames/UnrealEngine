// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "VariantManagerTestActor.generated.h"

#define UE_API VARIANTMANAGERCONTENTEDITOR_API

UENUM(BlueprintType)
enum class EVariantManagerTestEnum : uint8
{
	None = 0 UMETA(Hidden),
	FirstOption = 1,
	SecondOption = 3,
	ThirdOption = 45,
};

UCLASS(MinimalAPI, hideCategories=(Rendering, Physics, HLOD, Activation, Input, Actor, Cooking))
class AVariantManagerTestActor : public AActor
{
public:

	GENERATED_BODY()

	UE_API AVariantManagerTestActor(const FObjectInitializer& Init);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured byte property"))
	EVariantManagerTestEnum EnumWithNoDefault;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured byte property"))
	EVariantManagerTestEnum EnumWithSecondDefault = EVariantManagerTestEnum::SecondOption;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured byte property"))
	uint8 CapturedByteProperty;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured uint16 property"))
	//uint16 CapturedUInt16Property;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured uint32 property"))
	//uint32 CapturedUInt32Property;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured uint16 property"))
	//uint64 CapturedUInt64Property;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured int8 property"))
	//int8 CapturedInt8Property;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured int16 property"))
	//int16 CapturedInt16Property;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured int32 property"))
	int32 CapturedIntProperty = 7;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured int64 property"))
	//int64 CapturedInt64Property;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured float property"))
	float CapturedFloatProperty = 2.3f;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured double property"))
	//double CapturedDoubleProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured boolean property"))
	bool bCapturedBoolProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured UObject property"))
	TObjectPtr<UObject> CapturedObjectProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured UInterface property"))
	FScriptInterface CapturedInterfaceProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FName property"))
	FName CapturedNameProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FString property"))
	FString CapturedStrProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FText property"))
	FText CapturedTextProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FRotator property"))
	FRotator CapturedRotatorProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FColor property"))
	FColor CapturedColorProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FLinearColor property"))
	FLinearColor CapturedLinearColorProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FVector property"))
	FVector CapturedVectorProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FQuat property"))
	FQuat CapturedQuatProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FVector4 property"))
	FVector4 CapturedVector4Property;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FVector2D property"))
	FVector2D CapturedVector2DProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FIntPoint property"))
	FIntPoint CapturedIntPointProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured UObject array property"))
	TArray<TObjectPtr<UObject>> CapturedUObjectArrayProperty;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Template", meta=(ToolTip="Captured FVector array property"))
	TArray<FVector> CapturedVectorArrayProperty;
};

#undef UE_API
