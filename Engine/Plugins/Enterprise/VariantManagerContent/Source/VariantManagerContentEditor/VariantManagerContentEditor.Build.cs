// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class VariantManagerContentEditor : ModuleRules
	{
		public VariantManagerContentEditor(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"Engine",
					"ToolMenus",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore", // For ListView keyboard control
					"Projects",
					"Slate",
					"SlateCore",
					"VariantManagerContent",
					"WorkspaceMenuStructure",
				}
			);
		}
	}
}
