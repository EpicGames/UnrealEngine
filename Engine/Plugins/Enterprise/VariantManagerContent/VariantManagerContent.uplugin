{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Variant Manager Content",
	"Description": "Data classes and assets for the Variant Manager plugin",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules" :
	[
		{
			"Name": "VariantManagerContent",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "VariantManagerContentEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}